[CoreRedirects]
; AnimNextPoseSearch plugin -> UAFPoseSearch plugin
+PackageRedirects=(OldName="/Script/AnimNextPoseSearch", NewName="/Script/UAFPoseSearch")

+StructRedirects=(OldName="/Script/AnimNextPoseSearch.AnimNextHistoryCollectorTraitSharedData", NewName="/Script/UAFPoseSearch.AnimNextHistoryCollectorTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.AnimNextHistoryCollectorTask", NewName="/Script/UAFPoseSearch.AnimNextHistoryCollectorTask")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.AnimNextHistoryCollectorPreEvaluateTask", NewName="/Script/UAFPoseSearch.AnimNextHistoryCollectorPreEvaluateTask")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.MotionMatchingTraitSharedData", NewName="/Script/UAFPoseSearch.MotionMatchingTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.AnimNextMotionMatchingTask", NewName="/Script/UAFPoseSearch.AnimNextMotionMatchingTask")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.PoseHistoryAnimProperty", NewName="/Script/UAFPoseSearch.PoseHistoryAnimProperty")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.PoseSearchResultEmulatorTraitSharedData", NewName="/Script/UAFPoseSearch.PoseSearchResultEmulatorTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.AnimNextPoseSearchResultEmulatorTask", NewName="/Script/UAFPoseSearch.AnimNextPoseSearchResultEmulatorTask")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.RigUnit_DebugDrawTrajectory", NewName="/Script/UAFPoseSearch.RigUnit_DebugDrawTrajectory")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.RigUnit_MultiAnimGetAnimationAsset", NewName="/Script/UAFPoseSearch.RigUnit_MultiAnimGetAnimationAsset")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.RigUnit_PoseSearchDatabaseGetTags", NewName="/Script/UAFPoseSearch.RigUnit_PoseSearchDatabaseGetTags")
+StructRedirects=(OldName="/Script/AnimNextPoseSearch.RigUnit_PoseSearchResultGetSelectedDatabase", NewName="/Script/UAFPoseSearch.RigUnit_PoseSearchResultGetSelectedDatabase")

+ClassRedirects=(OldName="/Script/AnimNextPoseSearch.PoseSearchFeatureChannel_DistanceFromAnimNextVar", NewName="/Script/UAFPoseSearch.PoseSearchFeatureChannel_DistanceFromAnimNextVar")

; System references
+PropertyRedirects=(OldName="/Script/AnimNextPoseSearch.MotionMatchingTraitSharedData.MotionMatchingResultModuleHandle", NewName="MotionMatchingResultSystemReference")
+PropertyRedirects=(OldName="/Script/AnimNextPoseSearch.MotionMatchingTraitSharedData.MotionMatchingResultModuleHandleAlt", NewName="MotionMatchingResultSystemReferenceAlt")
