// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/BoneReference.h"
#include "Animation/TrajectoryTypes.h"
#include "CoreMinimal.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"
#include "TraitCore/TraitBinding.h"
#include "TraitCore/TraitSharedData.h"
#include "Variables/AnimNextVariableReference.h"
#include "HistoryCollectorTraitData.generated.h"

USTRUCT(meta = (DisplayName = "Pose History"))
struct FAnimNextHistoryCollectorTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// The maximum amount of poses that can be stored
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin="2"))
	int32 PoseCount = 2;
	
	// how often in seconds poses are collected (if 0, it will collect every update)
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin="0"))
	float SamplingInterval = 0.04f;

	UPROPERTY(EditAnywhere, Category = Settings)
	TArray<FBoneReference> CollectedBones;

	UPROPERTY(EditAnywhere, Category = Settings)
	TArray<FName> CollectedCurves;

	// Reset the pose history if it has become relevant to the graph after not being updated on previous frames.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bResetOnBecomingRelevant = true;

	// if true pose scales will be cached, otherwise implied to be unitary scales
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bStoreScales = false;

	// time in seconds to recover to the reference skeleton root bone transform by RootBoneTranslationRecoveryRatio and RootBoneRotationRecoveryRatio
	// from any eventual root bone modification. if zero the behaviour will be disabled (Experimental)
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (ClampMin="0"))
	float RootBoneRecoveryTime = 0.f;

	// ratio to recover to the reference skeleton root bone translation from any eventual root bone modification. zero for no recovery, 1 for full recovery
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (ClampMin="0", ClampMax="1", EditCondition = "RootBoneRecoveryTime > 0", EditConditionHides))
	float RootBoneTranslationRecoveryRatio = 1.f;

	// ratio to recover to the reference skeleton root bone rotation from any eventual root bone modification. zero for no recovery, 1 for full recovery
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (ClampMin="0", ClampMax="1", EditCondition = "RootBoneRecoveryTime > 0", EditConditionHides))
	float RootBoneRotationRecoveryRatio = 1.f;

//#if WITH_EDITORONLY_DATA
//	UPROPERTY(EditAnywhere, Category = Debug)
//	FLinearColor DebugColor = FLinearColor::Red;
//#endif // WITH_EDITORONLY_DATA

	// if true Trajectory the pose history node will generate the trajectory using the TrajectoryData parameters instead of relying on the input Trajectory (Experimental)
	UPROPERTY(EditAnywhere, Category = Experimental)
	bool bGenerateTrajectory = false;

	// input Trajectory samples for pose search queries in Motion Matching. These are expected to be in the world space of the SkeletalMeshComponent.
	// the trajectory sample with AccumulatedSeconds equals to zero (Trajectory.Samples[i].AccumulatedSeconds) is the sample of the previous frame of simulation (since MM works by matching the previous character pose)
	UPROPERTY(EditAnywhere, Transient, Category = Settings, meta = (PinShownByDefault, EditCondition="!bGenerateTrajectory", EditConditionHides))
	FTransformTrajectory Trajectory;

	// Input Trajectory velocity will be multiplied by TrajectorySpeedMultiplier: values below 1 will result in selecting animation slower than requested from the original Trajectory
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (PinHiddenByDefault, ClampMin="0", EditCondition="!bGenerateTrajectory", EditConditionHides))
	float TrajectorySpeedMultiplier = 1.f;

	// if bGenerateTrajectory is true, this is the number of trajectory past (collected) samples
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (ClampMin = "2", EditCondition = "bGenerateTrajectory", EditConditionHides))
	int32 TrajectoryHistoryCount = 10;

	// if bGenerateTrajectory is true, this is the number of trajectory future (prediction) samples
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (ClampMin="2", EditCondition="bGenerateTrajectory", EditConditionHides))
	int32 TrajectoryPredictionCount = 8;

	// if bGenerateTrajectory is true, this is the sampling interval between trajectory future (prediction) samples
	UPROPERTY(EditAnywhere, Category = Experimental, meta = (ClampMin="0.001", EditCondition="bGenerateTrajectory", EditConditionHides))
	float PredictionSamplingInterval = 0.4f;

	// Property to store a reference to the PoseHistory struct in
	UPROPERTY(EditAnywhere, Category = Experimental)
	FName PoseHistoryReferenceVariable;

	// if bGenerateTrajectory is true, TrajectoryData contains the tuning parameters to generate the trajectory
	UPROPERTY(EditAnywhere, Category = Experimental, meta=(EditCondition="bGenerateTrajectory", EditConditionHides))
	FPoseSearchTrajectoryData TrajectoryData;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(PoseCount) \
		GeneratorMacro(SamplingInterval) \
		GeneratorMacro(CollectedBones) \
		GeneratorMacro(CollectedCurves) \
		GeneratorMacro(bResetOnBecomingRelevant) \
		GeneratorMacro(bStoreScales) \
		GeneratorMacro(RootBoneRecoveryTime) \
		GeneratorMacro(RootBoneTranslationRecoveryRatio) \
		GeneratorMacro(RootBoneRotationRecoveryRatio) \
		GeneratorMacro(bGenerateTrajectory) \
		GeneratorMacro(Trajectory) \
		GeneratorMacro(TrajectorySpeedMultiplier) \
		GeneratorMacro(TrajectoryHistoryCount) \
		GeneratorMacro(TrajectoryPredictionCount) \
		GeneratorMacro(PredictionSamplingInterval) \
		GeneratorMacro(PoseHistoryReferenceVariable) \
		GeneratorMacro(TrajectoryData) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextHistoryCollectorTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};