// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "Module/UAFWeakSystemReference.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitCore/TraitBinding.h"
#include "TraitInterfaces/IGroupSynchronization.h"
#include "MotionMatchingTraitData.generated.h"

class UUAFBlendProfile;

USTRUCT(meta = (DisplayName = "Motion Matching"))
struct FMotionMatchingTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default", meta=(FactorySource))
	TArray<TObjectPtr<const UPoseSearchDatabase>> Databases;

	UPROPERTY(EditAnywhere, Category = "Default")
	FAlphaBlendArgs BlendArguments;

	// Don't jump to poses of the same segment that are within the interval this many seconds away from the continuing pose.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FFloatInterval PoseJumpThresholdTime = FFloatInterval(0.f, 0.f);

	/** The location on the x-axis to sample. Used if selection is a blend space. */
	UPROPERTY(EditAnywhere, Category = "Blend Space")
	float XAxisSamplePoint = 0.0f;

	/** The location on the y-axis to sample. Used if selection is a blend space. */
	UPROPERTY(EditAnywhere, Category = "Blend Space")
	float YAxisSamplePoint = 0.0f;

	// Prevent re-selection of poses that have been selected previously within this much time (in seconds) in the past. This is across all animation segments that have been selected within this time range.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin = "0"))
	float PoseReselectHistory = 0.3f;
	
	// If set to false, the motion matching node will perform a search only if the continuing pose is invalid. This is useful if you want to stagger searches of different nodes for performance reasons
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bShouldSearch = true;

	// Minimum amount of time to wait between searching for a new pose segment. It allows users to define how often the system searches, default for locomotion is searching every update, but you may only want to search once for other situations, like jump.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin="0"))
	float SearchThrottleTime = 0.f;

	// Effective range of play rate that can be applied to the animations to account for discrepancies in estimated velocity between the movement model and the animation.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault, ClampMin = "0.2", ClampMax = "3.0", UIMin = "0.2", UIMax = "3.0"))
	FFloatInterval PlayRate = FFloatInterval(1.f, 1.f);

	// If set to true, the search of multiple databases with different schemas will try to share pose features data calculated during query build
	// the idea is to be able to share as much as possible the continuing pose features vector across different schemas (and potentially improve performances)
	// defaulted to false to preserve behavior backward compatibility
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	bool bShouldUseCachedChannelData = false;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	FPoseSearchEvent EventToSearch;

	// If set to true, blend space inputs are continuously set based on input pins.
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bUpdateBlendSpaceInputs = false;

	// if bTrySkipBlendsForBlendSpaces is true and blendstack is already playing the currently selected blendspace, MM trait will try reusing the same graph by only changing the blend parameters
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bTrySkipBlendsForBlendSpaces = true;

	UPROPERTY(EditAnywhere, Category = Settings)
	float MaxDeltaAssetTimeToTrySkipBlendsForBlendSpaces = 0.1f;

	// The synchronization mode
	UPROPERTY(EditAnywhere, Category = Settings)
	EAnimGroupSynchronizationMode SyncMode = EAnimGroupSynchronizationMode::SynchronizeUsingUniqueGroupName;

	// @todo: should this be in FMotionMatchingState?
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	// Applied EPoseSearchInterruptMode on the next update that controls the continuing pose search evaluation. This is set back to EPoseSearchInterruptMode::DoNotInterrupt after each update.
	EPoseSearchInterruptMode NextUpdateInterruptMode = EPoseSearchInterruptMode::DoNotInterrupt;

	// If set, the motion matching result will be written to this variable
	UPROPERTY(EditAnywhere, Category = Settings)
	FName MotionMatchingResultVariableName = NAME_None;
	
	UPROPERTY(EditAnywhere, Category = Settings)
	FName AlignmentTransformVariableName;

	// If set along with result variable name, the motion matching result will be written to the variable in this system, otherwise the variable will be written in the module containing the MotionMatchingTrait
	UPROPERTY(EditAnywhere, Category = Settings)
	FUAFWeakSystemReference MotionMatchingResultSystemReference;

	// Same as MotionMatchingResultVariableName but for an alternative destination variable
	UPROPERTY(EditAnywhere, Category = Settings)
	FName MotionMatchingResultVariableNameAlt = NAME_None;

	// Same as MotionMatchingResultSystemReference but for an alternative destination variable
	UPROPERTY(EditAnywhere, Category = Settings)
	FUAFWeakSystemReference MotionMatchingResultSystemReferenceAlt;

	// Optional blend profile to use when new graph request is made
	UPROPERTY(EditAnywhere, Category = Settings, meta = (PinHiddenByDefault))
	TObjectPtr<UUAFBlendProfile> BlendProfile;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	TArray<FPoseSearchInteractionAvailability> Availabilities;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	bool bValidateResultAgainstAvailabilities = true;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	bool bKeepInteractionAlive = false;

	// Experimental, this feature might be removed without warning, not for production use
	// if bEnableWarp is true and this node is performing an interaction, warp aligment will be caluculated
	// and requested to be applied by the anim root motion provider (IAnimRootMotionProvider interface)
	// set it to false if a different warp aligment strategy is used
	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	bool bEnableWarp = false;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	bool bWarpUsingRootBone = true;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	float WarpingRotationRatio = 1.f;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	float WarpingTranslationRatio = 1.f;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	FName WarpingRotationCurveName;

	UPROPERTY(EditAnywhere, Category = Interaction, meta = (PinHiddenByDefault))
	FName WarpingTranslationCurveName;


	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(Databases) \
		GeneratorMacro(BlendArguments) \
		GeneratorMacro(PoseJumpThresholdTime) \
		GeneratorMacro(XAxisSamplePoint) \
		GeneratorMacro(YAxisSamplePoint) \
		GeneratorMacro(PoseReselectHistory) \
		GeneratorMacro(bShouldSearch) \
		GeneratorMacro(SearchThrottleTime) \
		GeneratorMacro(PlayRate) \
		GeneratorMacro(bShouldUseCachedChannelData) \
		GeneratorMacro(EventToSearch) \
		GeneratorMacro(bUpdateBlendSpaceInputs) \
		GeneratorMacro(bTrySkipBlendsForBlendSpaces) \
		GeneratorMacro(MaxDeltaAssetTimeToTrySkipBlendsForBlendSpaces) \
		GeneratorMacro(SyncMode) \
		GeneratorMacro(NextUpdateInterruptMode) \
		GeneratorMacro(MotionMatchingResultVariableName) \
		GeneratorMacro(AlignmentTransformVariableName) \
		GeneratorMacro(MotionMatchingResultSystemReference) \
		GeneratorMacro(MotionMatchingResultVariableNameAlt) \
		GeneratorMacro(MotionMatchingResultSystemReferenceAlt) \
		GeneratorMacro(BlendProfile) \
		GeneratorMacro(Availabilities) \
		GeneratorMacro(bValidateResultAgainstAvailabilities) \
		GeneratorMacro(bKeepInteractionAlive) \
		GeneratorMacro(bEnableWarp) \
		GeneratorMacro(bWarpUsingRootBone) \
		GeneratorMacro(WarpingRotationRatio) \
		GeneratorMacro(WarpingTranslationRatio) \
		GeneratorMacro(WarpingRotationCurveName) \
		GeneratorMacro(WarpingTranslationCurveName) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FMotionMatchingTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};