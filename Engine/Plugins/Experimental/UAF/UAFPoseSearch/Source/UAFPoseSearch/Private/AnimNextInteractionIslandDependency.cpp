// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextInteractionIslandDependency.h"

#include "Component/AnimNextComponent.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"

namespace UE::PoseSearch
{

FAnimNextInteractionIslandDependency FAnimNextInteractionIslandDependency::ModularFeature;

bool FAnimNextInteractionIslandDependency::CanMakeDependency(const UObject* InIslandObject, const UObject* InObject) const
{
	return InObject->IsA<UUAFComponent>();
}
	
const FTickFunction* FAnimNextInteractionIslandDependency::FindTickFunction(UObject* InObject) const
{
	const UUAFComponent* AnimNextComponent = CastChecked<UUAFComponent>(InObject);
	return AnimNextComponent->FindTickFunction(FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);
}

void FAnimNextInteractionIslandDependency::AddPrerequisite(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const
{
	UUAFComponent* AnimNextComponent = CastChecked<UUAFComponent>(InObject);
	AnimNextComponent->AddSubsequent(InIslandObject, InIslandTickFunction, FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);
}

void FAnimNextInteractionIslandDependency::AddSubsequent(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const
{
	UUAFComponent* AnimNextComponent = CastChecked<UUAFComponent>(InObject);
	AnimNextComponent->AddPrerequisite(InIslandObject, InIslandTickFunction, FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);
}

void FAnimNextInteractionIslandDependency::RemovePrerequisite(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const
{
	UUAFComponent* AnimNextComponent = CastChecked<UUAFComponent>(InObject);
	AnimNextComponent->RemoveSubsequent(InIslandObject, InIslandTickFunction, FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);
}

void FAnimNextInteractionIslandDependency::RemoveSubsequent(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const
{
	UUAFComponent* AnimNextComponent = CastChecked<UUAFComponent>(InObject);
	AnimNextComponent->RemovePrerequisite(InIslandObject, InIslandTickFunction, FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName);
}

}
