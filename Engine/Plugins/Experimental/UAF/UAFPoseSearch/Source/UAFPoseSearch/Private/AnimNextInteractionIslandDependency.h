// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchInteractionIsland.h"

namespace UE::UAF::PoseSearch
{
	class FModule;
}

namespace UE::PoseSearch
{

struct FAnimNextInteractionIslandDependency : public IInteractionIslandDependency
{
private:
	friend UE::UAF::PoseSearch::FModule;
	
	virtual bool CanMakeDependency(const UObject* InIslandObject, const UObject* InObject) const override;
	virtual const FTickFunction* FindTickFunction(UObject* InObject) const override;

	virtual void AddPrerequisite(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const override;
	virtual void AddSubsequent(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const override;
	virtual void RemovePrerequisite(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const override;
	virtual void RemoveSubsequent(UObject* InIslandObject, FTickFunction& InIslandTickFunction, UObject* InObject) const override;

	// Single impl for modular feature
	static FAnimNextInteractionIslandDependency ModularFeature;
};

}

