// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFPoseHistoryNode.h"
#include "PoseHistoryEvaluation.h"
#include "PoseSearch/PoseSearchContext.h"
#include "UAFAssetInstance.h"
#include "UAF/AnimOpCore/UAFAnimOpValueEvaluator.h"
#include "UAF/AnimOpCore/UAFAnimOp.h"
#include "UAF/AnimOps/UAFNullAnimOp.h"
#include "UAF/Attributes/EngineAttributes.h"

namespace UE::UAF
{
const FUAFStackName POSE_HISTORY_STACK_NAME = FName("Pose History Stack");
UE_UAF_REGISTER_STACK_TYPE(UE::PoseSearch::FPoseHistoryEvaluationHelper);

FUAFHistoryCollectorPreAnimOp::FUAFHistoryCollectorPreAnimOp()
	: FUAFAnimOp(0)
{
	InitializeAs<FUAFHistoryCollectorPreAnimOp>();
}

void FUAFHistoryCollectorPreAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	using namespace UE::PoseSearch;

	Evaluator.GetOrCreateStack<FPoseHistoryEvaluationHelper>(POSE_HISTORY_STACK_NAME).Push(FPoseHistoryEvaluationHelper{ PoseHistory });
}

// @todo: revisit this code once a FCSPose<FLODPoseStack> is implemented!
struct FUAFComponentSpacePoseProvider : public UE::PoseSearch::IComponentSpacePoseProvider
{
	FUAFComponentSpacePoseProvider(const FPoseValueBundle& InValueBundle, const USkeleton* InSkeleton)
		: ValueBundle(InValueBundle)
		, Skeleton(InSkeleton)
	{
		const FPoseValueBundlePtr DefaultValues = ValueBundle.GetNamedSet()->GetDefaultAttributeValues();

		DefaultBoneTransforms = DefaultValues->FindBoneTransforms();
		BoneTransforms = ValueBundle.FindBoneTransforms();
		DefaultBoneSet = DefaultBoneTransforms->GetTypedSet();
		BoneSet = BoneTransforms->GetTypedSet();

		const int32 NumBones = DefaultBoneTransforms->Num();
		ComponentSpaceFlags.SetNumZeroed(NumBones);
		ComponentSpaceTransforms.SetNum(NumBones);
	}

	virtual FTransform CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx) override
	{
		FTransform& ComponentSpaceTransform = ComponentSpaceTransforms[SkeletonBoneIdx.GetInt()];
		uint8& ComponentSpaceFlag = ComponentSpaceFlags[SkeletonBoneIdx.GetInt()];

		if (ComponentSpaceFlag)
		{
			return ComponentSpaceTransform;
		}

		const FAttributeBindingIndex BoneBindingIndex(SkeletonBoneIdx);

		const FAttributeSetIndex BoneDefaultSetIndex = DefaultBoneSet->GetIndex(BoneBindingIndex);
		const FAttributeSetIndex BoneSetIndex = BoneSet->GetIndex(BoneBindingIndex);

		if (BoneSetIndex.IsValid())
		{
			ComponentSpaceTransform = (*BoneTransforms)[BoneSetIndex].Value;
		}
		else
		{
			// @todo: use the skeletal mesh reference pose instead of the skeleton one to account for retargeting
			ComponentSpaceTransform = (*DefaultBoneTransforms)[BoneDefaultSetIndex].Value;
		}

		if (!BoneDefaultSetIndex.IsRootBone())
		{
			// If we aren't the root, apply our parent transform
			const FAttributeBindingIndex ParentBindingIndex = DefaultBoneSet->GetBindingIndex(DefaultBoneSet->GetParentIndex(BoneDefaultSetIndex));
			ComponentSpaceTransform *= CalculateComponentSpaceTransform(ParentBindingIndex);
		}

		ComponentSpaceFlag = 1;
		return ComponentSpaceTransform;
	}

	virtual const USkeleton* GetSkeletonAsset() const override
	{
		return Skeleton;
	}

private:
	const FPoseValueBundle& ValueBundle;
	const USkeleton* Skeleton;

	const TBoundValueMap<FBoneTransformAnimationAttribute>* DefaultBoneTransforms;
	const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms;
	FAttributeTypedSetPtr DefaultBoneSet;
	FAttributeTypedSetPtr BoneSet;

	TArray<uint8, TInlineAllocator<256, TMemStackAllocator<>>> ComponentSpaceFlags;
	TArray<FTransform, TInlineAllocator<256, TMemStackAllocator<>>> ComponentSpaceTransforms;
};

FUAFHistoryCollectorPostAnimOp::FUAFHistoryCollectorPostAnimOp()
	: FUAFAnimOp(1)
{
	InitializeAs<FUAFHistoryCollectorPostAnimOp>();
}

void FUAFHistoryCollectorPostAnimOp::EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator)
{
	using namespace UE::PoseSearch;

	const FUAFAnimOpValueEvaluationContext& AnimOpEvaluationContext = Evaluator.GetActiveEvaluationContext();

	const USkeleton* Skeleton = AnimOpEvaluationContext.GetSkeleton();
	FPoseValueBundleCoWRef* InputRef = Evaluator.GetEvaluationStack().PeekMutable();
	const FPoseValueBundle& Input = InputRef->Get();

	if (Input.IsEmpty())
	{
		// Our input is empty, fill it with the local space bind pose
		// Our CoW reference is already mutable if we are empty
		InputRef->GetMutable().InitWithValueSpace(FValueSpace(EValueSpaceType::Local));
	}

	bool bNeedsReset = false;
	bool bCacheBones = false;

	FMemMark Mark(FMemStack::Get());

	TArray<FBoneIndexType> BoneIndicesWithParents;

	if (PoseCount != PoseHistory->GetMaxNumPoses() || SamplingInterval != PoseHistory->GetSamplingInterval())
	{
		bCacheBones = true;
		PoseHistory->Initialize_AnyThread(PoseCount, SamplingInterval);
	}

	if (bCacheBones)
	{
		BoneIndicesWithParents.Add(UE::PoseSearch::RootBoneIndexType);

		for (const FBoneReference& CollectedBone : CollectedBones)
		{
			if (CollectedBone.BoneName != NAME_None)
			{
				FBoneReference TempCollectedBone = CollectedBone;
				TempCollectedBone.Initialize(Skeleton);
				if (TempCollectedBone.HasValidSetup())
				{
					BoneIndicesWithParents.AddUnique(TempCollectedBone.BoneIndex);
				}
			}
		}

		// Build separate index array with parent indices guaranteed to be present. Sort for EnsureParentsPresent.
		BoneIndicesWithParents.Sort();
		FAnimationRuntime::EnsureParentsPresent(BoneIndicesWithParents, Skeleton->GetReferenceSkeleton());
	}

	FUAFComponentSpacePoseProvider ComponentSpacePoseProvider(Input, Skeleton);
	PoseHistory->EvaluateComponentSpace_AnyThread(DeltaTime, ComponentSpacePoseProvider, bStoreScales,
		RootBoneRecoveryTime, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio,
		bNeedsReset, bCacheBones, BoneIndicesWithParents
		, FBlendedCurve(), CollectedCurves
#if WITH_EDITORONLY_DATA
		, AnimContext.IsValid() ? UE::PoseSearch::GetContextTransform(AnimContext.Pin().Get(), false) : FTransform::Identity
#endif //WITH_EDITORONLY_DATA
	);

	// Pop and discard
	(void)Evaluator.GetOrCreateStack<FPoseHistoryEvaluationHelper>(POSE_HISTORY_STACK_NAME).Pop();
}
	
FUAFPoseHistoryNode::FUAFPoseHistoryNode(FUAFAnimGraphUpdateContext& Context, const FUAFPoseHistoryNodeData& InData)
	: FUAFModifierAnimNode(Context)
	, Data(&InData)
{
	InitializeAs<FUAFPoseHistoryNode>(Context);
	InitializeModifier(Context, InData);

	if (HasChild())
	{
		PoseHistory = MakeShared<UE::PoseSearch::FGenerateTrajectoryPoseHistory>();

		PreAnimOp.PoseHistory = PoseHistory;
		PreAnimOp.SetDebugOwner(this);

		PostAnimOp.PoseHistory = PoseHistory;
		PostAnimOp.CollectedBones = InData.Settings.CollectedBones;
		PostAnimOp.CollectedCurves = InData.Settings.CollectedCurves;
		PostAnimOp.bStoreScales = InData.Settings.bStoreScales;
		PostAnimOp.PoseCount = InData.Settings.PoseCount;
		PostAnimOp.SamplingInterval = InData.Settings.SamplingInterval;
		PostAnimOp.HostObject = Context.GetHostObject();
		PostAnimOp.SetDebugOwner(this);

#if WITH_EDITORONLY_DATA
		PostAnimOp.AnimContext = Context.GetHostObject();
#endif // WITH_EDITORONLY_DATA

		SetPreAnimOp(&PreAnimOp);
		SetPostAnimOp(&PostAnimOp);
	}
}

void FUAFPoseHistoryNode::PreUpdate(FUAFAnimGraphUpdateContext& GraphContext)
{
	SCOPED_NAMED_EVENT(AnimNode_Update_PoseHistory, FColor::Blue);

	if (!HasChild())
	{
		// this should never happen...
		return;
	}

	if (Data->Trajectory.IsValid())
	{
		FTransformTrajectory TransformTrajectory;
		// probably need a way to get a pointer to trajectory to avoid array copies
		GraphContext.GetVariablesOwner()->GetVariable(Data->Trajectory, TransformTrajectory);
		PoseHistory->SetTrajectory(TransformTrajectory);
	}

	if (Data->PoseHistory.IsValid())
	{
		FPoseHistoryReference Reference;
		Reference.PoseHistory = PoseHistory;
		GraphContext.GetVariablesOwner()->SetVariable(Data->PoseHistory, Reference);
	}

	PostAnimOp.DeltaTime = GraphContext.GetDeltaTime();
}

FUAFAnimNodePtr FUAFPoseHistoryNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	return MakeAnimNode<FUAFPoseHistoryNode>(Context, *this);
}

void* FUAFPoseHistoryNodeData::GetInterface(FUAFAnimNodeInterfaceId Id)
{
	if (Child.IsValid())
	{
		return Child->GetInterface(Id);
	}

	return nullptr;
}

#if UAF_TRACE_ENABLED
	FString FUAFPoseHistoryNode::GetDebugName() const
	{
		static FString Name("Pose History");
		return Name;
	}

	UStruct* FUAFPoseHistoryNode::GetDebugStruct() const
	{
		return FUAFPoseHistoryNodeData::StaticStruct();
	}
#endif	
}
