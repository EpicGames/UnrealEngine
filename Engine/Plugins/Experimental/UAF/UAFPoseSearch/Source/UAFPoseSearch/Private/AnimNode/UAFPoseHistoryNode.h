// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HistoryCollectorTraitData.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNode.h"
#include "UAF/AnimNodeCore/UAFModifierAnimNodeData.h"

#include "UAFPoseHistoryNode.generated.h"

namespace UE::UAF
{
	
struct FUAFPoseHistoryNodeData;

USTRUCT()
struct FUAFHistoryCollectorPreAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFHistoryCollectorPreAnimOp)

	FUAFHistoryCollectorPreAnimOp();

	virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	TSharedPtr<UE::PoseSearch::FGenerateTrajectoryPoseHistory> PoseHistory;
};

USTRUCT()
struct FUAFHistoryCollectorPostAnimOp : public FUAFAnimOp
{
	GENERATED_BODY()
	UAF_DECLARE_ANIMOP(FUAFHistoryCollectorPostAnimOp)

	FUAFHistoryCollectorPostAnimOp();

	virtual void EvaluateValues(FUAFAnimOpValueEvaluator& Evaluator) override;

	TSharedPtr<UE::PoseSearch::FGenerateTrajectoryPoseHistory> PoseHistory;
	bool bStoreScales = false;
	float DeltaTime = 0.f;
	TWeakObjectPtr<const UObject> HostObject;

#if WITH_EDITORONLY_DATA
	TWeakObjectPtr<const UObject> AnimContext;
#endif // WITH_EDITORONLY_DATA

	int32 PoseCount = 0;
	float SamplingInterval = 0.f;
	float RootBoneRecoveryTime = 0.f;
	float RootBoneTranslationRecoveryRatio = 0.f;
	float RootBoneRotationRecoveryRatio = 0.f;

	// @todo: minimize allocations
	TArray<FBoneReference> CollectedBones;
	TArray<FName> CollectedCurves;
};

struct FUAFPoseHistoryNode : public FUAFModifierAnimNode
{
	FUAFPoseHistoryNode(FUAFAnimGraphUpdateContext& Context, const FUAFPoseHistoryNodeData& InData);

	// FUAFAnimNode impl
	virtual void PreUpdate(FUAFAnimGraphUpdateContext& GraphContext) override;
	
	#if UAF_TRACE_ENABLED
    	virtual FString GetDebugName() const override;
    	virtual UStruct* GetDebugStruct() const override;
    #endif

private:
	TSharedPtr<UE::PoseSearch::FGenerateTrajectoryPoseHistory> PoseHistory;
	
	FUAFHistoryCollectorPreAnimOp PreAnimOp;
	FUAFHistoryCollectorPostAnimOp PostAnimOp;

	const FUAFPoseHistoryNodeData* Data = nullptr;
};


USTRUCT(DisplayName = "Pose History")
struct FUAFPoseHistoryNodeData : public FUAFModifierAnimNodeData
{
	GENERATED_BODY()

	// todo: this should be some sort of output binding when we build that system
	UPROPERTY(EditAnywhere, Category = "NodeData")
	FAnimNextVariableReference PoseHistory;

	// todo: this should be some sort of input binding when we build that system
	UPROPERTY(EditAnywhere, Category = "NodeData")
	FAnimNextVariableReference Trajectory;

	// Use Trait Shared Data for settings to keep feature set in sync
	UPROPERTY(EditAnywhere, Category = "NodeData")
	FAnimNextHistoryCollectorTraitSharedData Settings;

	virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
	virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;
};
	
}