// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryCollectorTrait.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/Skeleton.h"
#include "AnimationRuntime.h"
#include "EvaluationVM/EvaluationVM.h"
#include "GenerationTools.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "PoseHistoryEvaluation.h"
#include "PoseSearch/PoseSearchContext.h"
#include "TraitCore/NodeInstance.h"

namespace UE::UAF
{

	namespace
	{
		template <typename ValueType>
		EPropertyBagResult SetVariableValue(const FUAFAssetInstance* Instance, const FAnimNextVariableReference& Variable, ValueType Value)
		{
			while(Instance)
			{
				if (Instance->SetVariable(Variable, Value) == EPropertyBagResult::Success)
				{
					return EPropertyBagResult::Success;
				}
		
				Instance = Instance->GetHost();
			}
	
			return EPropertyBagResult::PropertyNotFound;
		}
	}
	
	// @todo: revisit this code once a FCSPose<FLODPoseStack> is implemented!
	struct FComponentSpacePoseProvider : public UE::PoseSearch::IComponentSpacePoseProvider
	{
		FComponentSpacePoseProvider(const FLODPoseStack& InPose)
			: Pose(InPose)
		{
			const USkeleton* Skeleton = GetSkeletonInternal();
			const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
			const int32 NumBones = ReferenceSkeleton.GetNum();
			ComponentSpaceFlags.SetNumZeroed(NumBones);
			ComponentSpaceTransforms.SetNum(NumBones);
		}

		virtual FTransform CalculateComponentSpaceTransform(const FSkeletonPoseBoneIndex SkeletonBoneIdx) override
		{
			FTransform& ComponentSpaceTransform = ComponentSpaceTransforms[SkeletonBoneIdx.GetInt()];
			uint8& ComponentSpaceFlag = ComponentSpaceFlags[SkeletonBoneIdx.GetInt()];

			if (ComponentSpaceFlag)
			{
				return ComponentSpaceTransform;
			}

			const USkeleton* Skeleton = GetSkeletonInternal();
			const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();

			const TArrayView<const FBoneIndexType> SkeletonBoneIndexToLODBoneIndexMap = Pose.GetSkeletonBoneIndexToLODBoneIndexMap();
			const FBoneIndexType LODBoneIndex = SkeletonBoneIndexToLODBoneIndexMap[SkeletonBoneIdx.GetInt()];
			
			if (LODBoneIndex != INVALID_BONE_INDEX && LODBoneIndex < Pose.GetNumBones())
			{
				ComponentSpaceTransform = Pose.LocalTransformsView[LODBoneIndex];
			}
			else
			{
				// @todo: use the skeletal mesh reference pose instead of the skeleton one to account for retargeting
				ComponentSpaceTransform = ReferenceSkeleton.GetRefBonePose()[SkeletonBoneIdx.GetInt()];
			}
			
			const int32 ParentIndex = ReferenceSkeleton.GetParentIndex(SkeletonBoneIdx.GetInt());
			if (ParentIndex >= 0)
			{
				ComponentSpaceTransform *= CalculateComponentSpaceTransform(FSkeletonPoseBoneIndex(ParentIndex));
			}

			ComponentSpaceFlag = 1;
			return ComponentSpaceTransform;
		}

		virtual const USkeleton* GetSkeletonAsset() const override
		{
			return GetSkeletonInternal();
		}

	private:
		const USkeleton* GetSkeletonInternal() const
		{
			const USkeleton* Skeleton = Pose.RefPose->Skeleton.Get();
			check(Skeleton);
			return Skeleton;
		}

		const FLODPoseStack& Pose;
		TArray<uint8, TInlineAllocator<256 , TMemStackAllocator<>>> ComponentSpaceFlags;
		TArray<FTransform, TInlineAllocator<256 , TMemStackAllocator<>>> ComponentSpaceTransforms;
	};
}

void FAnimNextHistoryCollectorPreEvaluateTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::PoseSearch;

	if (VM.GetActiveNamedSet())
	{
		FPoseHistoryEvaluationHelper PoseHistoryEvalHelper;
		PoseHistoryEvalHelper.PoseHistoryPtr = InstanceData->PoseHistoryPtr;
		VM.PushValue(POSEHISTORY_STACK_NAME, MakeUnique<FPoseHistoryEvaluationHelper>(PoseHistoryEvalHelper));
	}
	else
	{
		if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
		{
			FPoseHistoryEvaluationHelper PoseHistoryEvalHelper;
			PoseHistoryEvalHelper.PoseHistoryPtr = InstanceData->PoseHistoryPtr;
			VM.PushValue(POSEHISTORY_STACK_NAME, MakeUnique<FPoseHistoryEvaluationHelper>(PoseHistoryEvalHelper));
		}
	}
}

void FAnimNextHistoryCollectorTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		if (TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValueMutable<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
		{
			bool bNeedsReset = false;
			bool bCacheBones = false;

			FMemMark Mark(FMemStack::Get());

			const FLODPoseStack& Pose = (*Keyframe)->Pose;
			TArray<FBoneIndexType> BoneIndicesWithParents;

			if (PoseCount != PoseHistory->GetMaxNumPoses() || SamplingInterval != PoseHistory->GetSamplingInterval())
			{
				bCacheBones = true;
				PoseHistory->Initialize_AnyThread(PoseCount, SamplingInterval);
			}

			if (bCacheBones)
			{
				const USkeleton* Skeleton = Pose.RefPose->Skeleton.Get();
				check(Skeleton);

				BoneIndicesWithParents.Add(UE::PoseSearch::RootBoneIndexType);

				for (const FBoneReference& CollectedBone : CollectedBones)
				{
					if (CollectedBone.BoneName != NAME_None)
					{
						FBoneReference TempCollectedBone = CollectedBone;
						TempCollectedBone.Initialize(Skeleton);
						if (TempCollectedBone.HasValidSetup())
						{
							BoneIndicesWithParents.AddUnique(TempCollectedBone.BoneIndex);
						}
					}
				}

				// Build separate index array with parent indices guaranteed to be present. Sort for EnsureParentsPresent.
				BoneIndicesWithParents.Sort();
				FAnimationRuntime::EnsureParentsPresent(BoneIndicesWithParents, Skeleton->GetReferenceSkeleton());
			}

			const bool bHasCurves = EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves);

			FComponentSpacePoseProvider ComponentSpacePoseProvider(Pose);
			PoseHistory->EvaluateComponentSpace_AnyThread(DeltaTime, ComponentSpacePoseProvider, bStoreScales,
				RootBoneRecoveryTime, RootBoneTranslationRecoveryRatio, RootBoneRotationRecoveryRatio,
				bNeedsReset, bCacheBones, BoneIndicesWithParents
				, bHasCurves ? (*Keyframe)->Curves : FBlendedCurve(), CollectedCurves
#if WITH_EDITORONLY_DATA
				, AnimContext.IsValid() ? UE::PoseSearch::GetContextTransform(AnimContext.Pin().Get(), false) : FTransform::Identity
#endif //WITH_EDITORONLY_DATA
			);
		}
	}
}

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FHistoryCollectorTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IPoseHistory) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FHistoryCollectorTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FHistoryCollectorTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		
#if WITH_EDITOR
		if (InstanceData->bIsPostEvaluateBeingCalled)
		{
			InstanceData->bIsPostEvaluateBeingCalled = false;
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FHistoryCollectorTrait::PreUpdate, PostEvaluate has not being called last frame! Some trait in the TraitStack didn't propagate correctly the PostEvaluate!");
		}
#endif // WITH_EDITOR

		InstanceData->DeltaTime = TraitState.GetDeltaTime();

		// @todo: implement bResetOnBecomingRelevant
		//bool bResetOnBecomingRelevant = SharedData->GetbResetOnBecomingRelevant(Binding);

		check(InstanceData->PoseHistoryPtr.IsValid());
		InstanceData->PoseHistoryPtr->bGenerateTrajectory = SharedData->GetbGenerateTrajectory(Binding);
		if (InstanceData->PoseHistoryPtr->bGenerateTrajectory)
		{
			const UObject* AnimContext = nullptr;
			if (const FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
			{
				AnimContext = ModuleInstance->GetObject();
			}

			InstanceData->PoseHistoryPtr->TrajectoryDataSampling.NumHistorySamples = FMath::Max(SharedData->GetPoseCount(Binding), SharedData->GetTrajectoryHistoryCount(Binding));
			InstanceData->PoseHistoryPtr->TrajectoryDataSampling.SecondsPerHistorySample = SharedData->GetSamplingInterval(Binding);
			InstanceData->PoseHistoryPtr->TrajectoryDataSampling.NumPredictionSamples = SharedData->GetTrajectoryPredictionCount(Binding);
			InstanceData->PoseHistoryPtr->TrajectoryDataSampling.SecondsPerPredictionSample = SharedData->GetPredictionSamplingInterval(Binding);
			InstanceData->PoseHistoryPtr->TrajectoryData = SharedData->GetTrajectoryData(Binding);

			InstanceData->PoseHistoryPtr->GenerateTrajectory(AnimContext, InstanceData->DeltaTime);
		}
		else
		{
			InstanceData->PoseHistoryPtr->SetTrajectory(SharedData->GetTrajectory(Binding), SharedData->GetTrajectorySpeedMultiplier(Binding));
		}

		InstanceData->PoseHistoryPtr->bIsTrajectoryGeneratedBeforePreUpdate = false;

		Context.PushScopedInterface<IPoseHistory>(Binding);

		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FHistoryCollectorTrait::PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		ensure(Context.PopScopedInterface<IPoseHistory>(Binding));
		IUpdate::PostUpdate(Context, Binding, TraitState);
	}

	void FHistoryCollectorTrait::PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PreEvaluate(Context, Binding);
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		check(InstanceData->PoseHistoryPtr.IsValid());
		FAnimNextHistoryCollectorPreEvaluateTask Task;
		Task.InstanceData = InstanceData;
		Context.AppendTask(Task);
	}

	void FHistoryCollectorTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

#if WITH_EDITOR
		if (InstanceData->bIsPostEvaluateBeingCalled)
		{
			UE_LOGF(LogPoseSearch, Error, "FHistoryCollectorTrait::PostEvaluate, PostEvaluate called without calling PreUpdate on this frame! Some trait in the TraitStack doesn't propagate correctly the PreUpdate!");
		}
		else
		{
			InstanceData->bIsPostEvaluateBeingCalled = true;
		}
#endif // WITH_EDITOR

		const FName& PoseHistoryReferenceVariableName = SharedData->GetPoseHistoryReferenceVariable(Binding);
		if (PoseHistoryReferenceVariableName != NAME_None)
		{
			FPoseHistoryReference Reference;
			Reference.PoseHistory = InstanceData->PoseHistoryPtr;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (SetVariableValue(&Binding.GetTraitPtr().GetNodeInstance()->GetOwner(), FAnimNextVariableReference(PoseHistoryReferenceVariableName), Reference) != EPropertyBagResult::Success)
			{
				if (SetVariableValue(&Binding.GetTraitPtr().GetNodeInstance()->GetOwner(), FAnimNextVariableReference(PoseHistoryReferenceVariableName), InstanceData->PoseHistoryPtr->GetPoseSearchHistory()) != EPropertyBagResult::Success)
				{
					UE_LOGF(LogPoseSearch, Error, "FHistoryCollectorTrait::PostEvaluate, failed to set pose history to variable '%ls' (variable must be of type FPoseHistoryReference or FPoseSearchHistory)", *PoseHistoryReferenceVariableName.ToString());
				}
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
		check(InstanceData->PoseHistoryPtr.IsValid());
		FAnimNextHistoryCollectorTask Task;
		Task.PoseHistory = InstanceData->PoseHistoryPtr.Get();
		Task.bStoreScales = SharedData->GetbStoreScales(Binding);
		Task.DeltaTime = InstanceData->DeltaTime;
		Task.HostObject = Context.GetHostObject();
		Task.PoseCount = SharedData->GetPoseCount(Binding);
		Task.SamplingInterval = SharedData->GetSamplingInterval(Binding);
		Task.RootBoneRecoveryTime = SharedData->GetRootBoneRecoveryTime(Binding);
		Task.RootBoneTranslationRecoveryRatio = SharedData->GetRootBoneTranslationRecoveryRatio(Binding);
		Task.RootBoneRotationRecoveryRatio = SharedData->GetRootBoneRotationRecoveryRatio(Binding);
		Task.CollectedBones = SharedData->GetCollectedBones(Binding);
		Task.CollectedCurves = SharedData->GetCollectedCurves(Binding);

#if WITH_EDITORONLY_DATA
		if (const FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
		{
			Task.AnimContext = ModuleInstance->GetObject();
		}
#endif // WITH_EDITORONLY_DATA

		Context.AppendTask(Task);
	}

	const UE::PoseSearch::IPoseHistory* FHistoryCollectorTrait::GetPoseHistory(FExecutionContext& Context, const TTraitBinding<IPoseHistory>& Binding) const
	{
		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData->PoseHistoryPtr.IsValid());
		return InstanceData->PoseHistoryPtr.Get();
	}
}
