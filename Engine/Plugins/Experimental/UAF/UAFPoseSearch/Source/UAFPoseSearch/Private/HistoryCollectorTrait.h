// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPoseHistory.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "Animation/TrajectoryTypes.h"
#include "TraitCore/Trait.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "EvaluationVM/EvaluationVM.h"
#include "HistoryCollectorTraitData.h"
#include "HistoryCollectorTrait.generated.h"

USTRUCT()
struct FAnimNextHistoryCollectorTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextHistoryCollectorTask)

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UE::PoseSearch::FPoseHistory* PoseHistory = nullptr;
	bool bStoreScales = false;
	float DeltaTime = 0.f;
	TWeakObjectPtr<const UObject> HostObject;
#if WITH_EDITORONLY_DATA
	TWeakObjectPtr<const UObject> AnimContext;
#endif // WITH_EDITORONLY_DATA

	int32 PoseCount = 0;
	float SamplingInterval = 0.f;
	float RootBoneRecoveryTime = 0.f;
	float RootBoneTranslationRecoveryRatio = 0.f;
	float RootBoneRotationRecoveryRatio = 0.f;

	// @todo: minimize allocations
	TArray<FBoneReference> CollectedBones;
	TArray<FName> CollectedCurves;
};

namespace UE::UAF
{
	struct FHistoryCollectorTrait : FAdditiveTrait, IUpdate, IEvaluate, IPoseHistory
	{
		DECLARE_ANIM_TRAIT(FHistoryCollectorTrait, FAdditiveTrait)

		using FSharedData = FAnimNextHistoryCollectorTraitSharedData;

		struct FInstanceData : FTrait::FInstanceData
		{
			FInstanceData() : PoseHistoryPtr(MakeShareable(new UE::PoseSearch::FGenerateTrajectoryPoseHistory())) { }
			TSharedPtr<UE::PoseSearch::FGenerateTrajectoryPoseHistory, ESPMode::ThreadSafe> PoseHistoryPtr;
			float DeltaTime = 0.f;

#if WITH_EDITOR
			bool bIsPostEvaluateBeingCalled = true;
#endif // WITH_EDITOR
		};

		// IUpdate impl
		virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
		virtual void PostUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

		// IEvaluate impl
		virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		//@TODO: this is a HACK. We're pushing a task with the pose history so that evaluation modifiers can see the pose history.
		// "Evaluation" modifiers should have an Update callback with Scoped Interface access, just like any other trait.
		virtual void PreEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

		// IPoseHistory impl
		virtual const UE::PoseSearch::IPoseHistory* GetPoseHistory(FExecutionContext& Context, const TTraitBinding<IPoseHistory>& Binding) const override;
	};
}

USTRUCT()
struct FAnimNextHistoryCollectorPreEvaluateTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextHistoryCollectorPreEvaluateTask)

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
	UE::UAF::FHistoryCollectorTrait::FInstanceData* InstanceData = nullptr;
};