// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPoseHistory.h"
#include "TraitCore/ExecutionContext.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT_INTERFACE(IPoseHistory)

#if WITH_EDITOR
	const FText& IPoseHistory::GetDisplayName() const
	{
		static FText InterfaceName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IPoseHistory_Name", "Pose History");
		return InterfaceName;
	}
	const FText& IPoseHistory::GetDisplayShortName() const
	{
		static FText InterfaceShortName = NSLOCTEXT("TraitInterfaces", "TraitInterface_IPoseHistory_ShortName", "PH");
		return InterfaceShortName;
	}
#endif // WITH_EDITOR

	const UE::PoseSearch::IPoseHistory* IPoseHistory::GetPoseHistory(FExecutionContext& Context, const TTraitBinding<IPoseHistory>& Binding) const
	{
		TTraitBinding<IPoseHistory> SuperBinding;
		if (Binding.GetStackInterfaceSuper(SuperBinding))
		{
			return SuperBinding.GetPoseHistory(Context);
		}

		return nullptr;
	}
}
