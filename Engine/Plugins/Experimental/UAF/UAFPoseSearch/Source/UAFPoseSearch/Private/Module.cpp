// Copyright Epic Games, Inc. All Rights Reserved.

#include "AlphaBlend.h"
#include "Animation/TrajectoryTypes.h"
#include "AnimNextInteractionIslandDependency.h"
#include "CoreMinimal.h"
#include "Features/IModularFeatures.h"
#include "Module/AnimNextModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchInteractionAsset.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchInteractionAlignment.h"
#include "PoseSearchSteerAlongTrajectory.h"
#include "RigVMCore/RigVMRegistry.h"

namespace UE::UAF::PoseSearch
{

class FModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
		{
			{ UPoseSearchDatabase::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UPoseSearchSchema::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UPoseSearchInteractionAsset::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UMultiAnimAsset::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class }
		};

		static UScriptStruct* const AllowedStructTypes[] =
		{
			FTransformTrajectorySample::StaticStruct(),
			FTransformTrajectory::StaticStruct(),
			FPoseSearchBlueprintResult::StaticStruct(),
			FAlphaBlendArgs::StaticStruct(),
			FPoseSearchInteractionAssetItem::StaticStruct(),
			FPoseSearchInteractionAvailability::StaticStruct(),
			FPoseHistoryReference::StaticStruct(),
			FPoseSearchHistory::StaticStruct(),
			FPoseSearchEvent::StaticStruct(),
		};

		FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
		RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);
		RigVMRegistry.RegisterStructTypes(AllowedStructTypes);

		IModularFeatures::Get().RegisterModularFeature(UE::PoseSearch::IInteractionIslandDependency::FeatureName, &UE::PoseSearch::FAnimNextInteractionIslandDependency::ModularFeature);

		UE::UAF::FEvaluationNotifiesTrait::RegisterEvaluationHandler(UNotifyState_PoseSearchInteractionAlignment::StaticClass(), FEvaluationNotify_PoseSearchInteractionAlignment::StaticStruct());
		UE::UAF::FEvaluationNotifiesTrait::RegisterEvaluationHandler(UNotifyState_PoseSearchSteerAlongTrajectory::StaticClass(), FEvaluationNotify_PoseSearchSteerAlongTrajectory::StaticStruct());
	}

	virtual void ShutdownModule() override
	{
		UE::UAF::FEvaluationNotifiesTrait::UnregisterEvaluationHandler(UNotifyState_PoseSearchSteerAlongTrajectory::StaticClass());
		UE::UAF::FEvaluationNotifiesTrait::UnregisterEvaluationHandler(UNotifyState_PoseSearchInteractionAlignment::StaticClass());

		IModularFeatures::Get().UnregisterModularFeature(UE::PoseSearch::IInteractionIslandDependency::FeatureName, &UE::PoseSearch::FAnimNextInteractionIslandDependency::ModularFeature);
	}
};

}

IMPLEMENT_MODULE(UE::UAF::PoseSearch::FModule, UAFPoseSearch)
