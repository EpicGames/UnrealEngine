// Copyright Epic Games, Inc. All Rights Reserved.

#include "MotionMatchingTrait.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "AnimNext/EvaluationNotifiesTrait.h"
#include "Component/AnimNextComponent.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Factory/AnimGraphFactory.h"
#include "Graph/AnimNextGraphInstance.h"
#include "IPoseHistory.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchDerivedData.h"
#include "PoseSearch/PoseSearchInteractionUtils.h"
#include "TraitCore/NodeInstance.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/ITimeline.h"
#include "Traits/BlendSpacePlayerTraitData.h"
#include "Traits/SequencePlayerTraitData.h"
#include "Traits/SynchronizeUsingGroupsTraitData.h"
#include "UAF/UAFAssetFactory.h"
#include "VisualLogger/VisualLogger.h"

#if ENABLE_ANIM_DEBUG
namespace UE::Private
{
	enum EPlayRateState : int8
	{
		Disabled = 0,
		Enabled = 1,
		PoseSearchOnly = 2
	};
} // namespace UE::Private

static int32 GVarAnimNextMotionMatchingPlayRateEnabled = UE::Private::EPlayRateState::Enabled;
static FAutoConsoleVariableRef CVarAnimNextMotionMatchingPlayRateEnabled(
	TEXT("a.AnimNext.MotionMatchingTrait.DebugPlayRateEnabled"),
	GVarAnimNextMotionMatchingPlayRateEnabled,
	TEXT("Toggles if PlayRate is used in motion matching. Same as setting PlayRate to (1,1) when disabled.\n")
	TEXT("0: Completely disable PlayRate usage.\n")
	TEXT("1: Enable all usages of PlayRate.\n")
	TEXT("2: Enable PlayRate in PoseSeach only (Not used in actual playback).\n")
);
#endif // ENABLE_ANIM_DEBUG

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FMotionMatchingTrait)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \
		GeneratorMacro(IGarbageCollection) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FMotionMatchingTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	void FMotionMatchingTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		IUpdate::OnBecomeRelevant(Context, Binding, TraitState);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);
	}

	void FMotionMatchingTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		using namespace UE::PoseSearch;

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

#if WITH_EDITOR
		if (InstanceData->bIsPostEvaluateBeingCalled)
		{
			InstanceData->bIsPostEvaluateBeingCalled = false;
		}
		else
		{
			UE_LOGF(LogPoseSearch, Error, "FMotionMatchingTrait::PreUpdate, PostEvaluate has not being called last frame! Some trait in the TraitStack didn't propagate correctly the PostEvaluate!");
		}
#endif // WITH_EDITOR

		TTraitBinding<IBlendStack> BlendStackBinding;
		if (!Binding.GetStackInterface(BlendStackBinding))
		{
			UE_LOGF(LogPoseSearch, Error, "FMotionMatchingTrait::PreUpdate, missing IBlendStack");
			return;
		}

		TTraitBinding<IPoseHistory> PoseHistoryTrait;
		if (!Context.GetScopedInterface<IPoseHistory>(PoseHistoryTrait))
		{
			UE_LOGF(LogPoseSearch, Error, "FMotionMatchingTrait::PreUpdate, missing UE::UAF::IPoseHistory");
			return;
		}

		TTraitBinding<ITimeline> TimelineTrait;
		if (!Binding.GetStackInterface(TimelineTrait))
		{
			UE_LOGF(LogPoseSearch, Error, "FMotionMatchingTrait::PreUpdate, missing ITimeline");
			return;
		}

		const UE::PoseSearch::IPoseHistory* PoseHistory = PoseHistoryTrait.GetPoseHistory(Context);
		if (!PoseHistory)
		{
			UE_LOGF(LogPoseSearch, Error, "FMotionMatchingTrait::PreUpdate, missing UE::PoseSearch::IPoseHistory");
			return;
		}

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const UObject* AnimContext = nullptr;
		if (const FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
		{
			AnimContext = ModuleInstance->GetObject();
		}

		FMotionMatchingState& MotionMatchingState = InstanceData->MotionMatchingState;

		// synchronizing with GetAccumulatedTime or resetting MotionMatchingState, and conditionally resetting FAnimNode_BlendStack_Standalone
		// @todo: implement this MotionMatchingState.Reset() condition for parity to FAnimNode_MotionMatching::UpdateAssetPlayer
		//if (bResetOnBecomingRelevant && UpdateCounter.HasEverBeenUpdated() && !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter()))
		//{
		//	// If we just became relevant and haven't been initialized yet, then reset motion matching state, otherwise update the asset time using the player node.
		//	MotionMatchingState.Reset();
		//	FAnimNode_BlendStack_Standalone::Reset();
		//}
		//else 
		if (MotionMatchingState.SearchResult.SelectedDatabase == nullptr || MotionMatchingState.SearchResult.SelectedDatabase->Schema == nullptr)
		{
		}
#if WITH_EDITOR
		else if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(MotionMatchingState.SearchResult.SelectedDatabase.Get(), ERequestAsyncBuildFlag::ContinueRequest))
		{
			// MotionMatchingState.SearchResult.Database is indexing, and it's not safe to use its previous index properties cached in MotionMatchingState
			MotionMatchingState.Reset();
		}
#endif // WITH_EDITOR

		const float DeltaTime = TraitState.GetDeltaTime();
		const TArrayView<const FPoseSearchInteractionAvailability> Availabilities = SharedData->GetAvailabilities(Binding);
		// MotionMatchInteraction synchronizes and integrates MotionMatchingState.SearchResult internally, so there's no need to update it with the blendstack information
		UPoseSearchInteractionLibrary::MotionMatchInteraction(MotionMatchingState.SearchResult, Availabilities, AnimContext, FName(), PoseHistory,
			SharedData->GetbValidateResultAgainstAvailabilities(Binding), SharedData->GetbKeepInteractionAlive(Binding), SharedData->GetBlendArguments(Binding).BlendTime, DeltaTime);

		// performing the regular single character motion matching search in case there's no MM interaction
		if (!MotionMatchingState.SearchResult.bIsInteraction)
		{
			// We adjust the motion matching state asset time to the current player node's asset time. This is done 
			// because the player node may have ticked more or less time than we expected due to variable dt or the 
			// dynamic playback rate adjustment and as such the motion matching state does not update by itself
			float TimelineNormalizedTime = 0.0f;
			FTimelineState TimelineState;
			if (TimelineTrait.GetState(Context, TimelineState))
			{
				const float TimelineRealTime = TimelineState.GetPosition();
				TimelineNormalizedTime = TimelineRealTime;

				// @todo: This is a hack, since we don't have a way to get the normalized timeline of a blendspace yet (pending syncing...?)
				if (Cast<UBlendSpace>(MotionMatchingState.SearchResult.GetAnimationAssetForRole()))
				{
					// Convert to normalized time.
					const float TimelineDuration = TimelineState.GetDuration();
					if (TimelineDuration > 0.0f)
					{
						// NOTE: This doesn't work because AnimNext normalized time does not match database indexing normalized time.
						TimelineNormalizedTime = TimelineRealTime / TimelineDuration;
					}
				}

#if ENABLE_VISUAL_LOG
				if (FVisualLogger::IsRecording())
				{
					const FName AnimName = TimelineState.GetDebugName();
					const float Duration = TimelineState.GetDuration();
					static const TCHAR* LogName = TEXT("FMotionMatchingTrait");
					UE_VLOG(Context.GetHostObject(), LogName, Verbose, TEXT("TimelineRealTime: %f"), TimelineRealTime);
					UE_VLOG(Context.GetHostObject(), LogName, Verbose, TEXT("TimelineNormalizedTime: %f"), TimelineNormalizedTime);
					UE_VLOG(Context.GetHostObject(), LogName, Verbose, TEXT("AnimName: %s"), *AnimName.ToString());
					UE_VLOG(Context.GetHostObject(), LogName, Verbose, TEXT("Duration: %f"), Duration);
				}
#endif
			}
			// @todo: ask blendstack what it's playing instead of relying on the previous MotionMatchingState.SearchResult
			// We adjust the motion matching state asset time to the current player node's asset time. This is done 
			// because the player node may have ticked more or less time than we expected due to variable dt or the 
			// dynamic playback rate adjustment and as such the motion matching state does not update by itself
			//MotionMatchingState.SearchResult.SelectedAnim = GetAnimAsset();
			MotionMatchingState.SearchResult.SelectedTime = TimelineNormalizedTime;
			//MotionMatchingState.SearchResult.bIsMirrored = GetMirror();
			//MotionMatchingState.SearchResult.BlendParameters = GetBlendParameters();

			const TConstArrayView<TObjectPtr<const UPoseSearchDatabase>> DatabasesToSearch = SharedData->GetDatabases(Binding);
			
			FFloatInterval PoseSearchPlayRate = SharedData->GetPlayRate(Binding);
#if ENABLE_ANIM_DEBUG
			bool bPoseSearchPlayRateEnabled = GVarAnimNextMotionMatchingPlayRateEnabled == UE::Private::EPlayRateState::Enabled || GVarAnimNextMotionMatchingPlayRateEnabled == UE::Private::EPlayRateState::PoseSearchOnly;
			PoseSearchPlayRate = bPoseSearchPlayRateEnabled ? PoseSearchPlayRate : FFloatInterval(1.f, 1.f);
#endif // ENABLE_ANIM_DEBUG

			const float SearchThrottleTime = SharedData->GetbShouldSearch(Binding) ? SharedData->GetSearchThrottleTime(Binding) : UE_BIG_NUMBER;

			FPoseSearchEvent EventToSearch = SharedData->GetEventToSearch(Binding);
			FSearchContext SearchContext(0.f, SharedData->GetPoseJumpThresholdTime(Binding), EventToSearch.GetPlayRateOverriddenEvent(PoseSearchPlayRate));

			FChooserEvaluationContext ChooserEvaluationContext(const_cast<UObject*>(AnimContext));
			const FUAFAssetInstance& Instance = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
			ChooserEvaluationContext.AddStructViewParam(FStructView::Make(const_cast<FUAFAssetInstance&>(Instance)));
			SearchContext.AddRole(GetCommonDefaultRole(DatabasesToSearch), &ChooserEvaluationContext, PoseHistory);

			// If we can't advance or enough time has elapsed since the last pose jump then search
			bool bSearch = true;
			FSearchResult SearchResult;
			if (MotionMatchingState.ElapsedPoseSearchTime < SearchThrottleTime)
			{
				SearchResult.InitFrom(MotionMatchingState.SearchResult);
				SearchContext.UpdateContinuingPoseSearchResult(SearchResult, SearchResult);
				const bool bCanAdvance = SearchContext.GetContinuingPoseSearchResult().PoseIdx != INDEX_NONE;
				bSearch = !bCanAdvance;
			}

			if (bSearch)
			{
				MotionMatchingState.ElapsedPoseSearchTime = 0.f;
			
				TArrayView<const UObject*> AssetsToSearch((const UObject**)FMemory_Alloca(DatabasesToSearch.Num() * sizeof(UObject*)), (int32)DatabasesToSearch.Num());
				for (int32 DatabaseToSearchIndex = 0; DatabaseToSearchIndex < DatabasesToSearch.Num(); ++DatabaseToSearchIndex)
				{
					AssetsToSearch[DatabaseToSearchIndex] = DatabasesToSearch[DatabaseToSearchIndex].Get();
				}

				FPoseSearchContinuingProperties ContinuingProperties;
				ContinuingProperties.InitFrom(MotionMatchingState.SearchResult, SharedData->GetNextUpdateInterruptMode(Binding));
				FSearchResults_Single SearchResults;
				UPoseSearchLibrary::MotionMatch(SearchContext, AssetsToSearch, ContinuingProperties, SearchResults);
				SearchResult = SearchResults.GetBestResult();
			}
			else
			{
				MotionMatchingState.ElapsedPoseSearchTime += DeltaTime;

				// if we didn't perform a search we carry on with the continuing pose search result
				SearchResult = SearchContext.GetContinuingPoseSearchResult();
				SearchResult.bIsContinuingPoseSearch = true;

#if UE_POSE_SEARCH_TRACE_ENABLED
				// in case we skipped the search, we still have to track we would have requested to evaluate DatabasesToSearch and SearchResult.Database
				for (const TObjectPtr<const UPoseSearchDatabase>& DatabaseToSearch : DatabasesToSearch)
				{
					SearchContext.Track(DatabaseToSearch.Get());
				}

				SearchContext.Track(SearchResult.Database.Get());
#endif // UE_POSE_SEARCH_TRACE_ENABLED
			}

			const float WantedPlayRate = CalculateWantedPlayRate(SearchResult, SearchContext, PoseSearchPlayRate, PoseHistory ? PoseHistory->GetTrajectorySpeedMultiplier() : 1.f, EventToSearch);
			MotionMatchingState.SearchResult.InitFrom(SearchResult, WantedPlayRate);

			if (const FPoseIndicesHistory* PoseIndicesHistory = PoseHistory->GetPoseIndicesHistory())
			{
				// const casting here is safe since we're in the thread owning the pose history, and it's the correct place to update the previously selected poses
				const_cast<FPoseIndicesHistory*>(PoseIndicesHistory)->Update(SearchResult, DeltaTime, SharedData->GetPoseReselectHistory(Binding));
			}
		}

		float DesiredPlayRate = MotionMatchingState.SearchResult.WantedPlayRate;
#if ENABLE_ANIM_DEBUG
		DesiredPlayRate = GVarAnimNextMotionMatchingPlayRateEnabled == UE::Private::EPlayRateState::Enabled ? DesiredPlayRate : 1.0f;
#endif // ENABLE_ANIM_DEBUG

		const bool bJumpToPose = MotionMatchingState.SearchResult.SelectedAnim && !MotionMatchingState.SearchResult.bIsContinuingPoseSearch;
		if (bJumpToPose)
		{
			if (UAnimationAsset* AnimationAsset = MotionMatchingState.SearchResult.GetAnimationAssetForRole())
			{
				const bool bTrySkipBlendsForBlendSpaces = SharedData->GetbTrySkipBlendsForBlendSpaces(Binding);
				const bool bIsBlendSpace = AnimationAsset->IsA<UBlendSpace>();
				bool bSkipBlend = false;
				if (bIsBlendSpace && bTrySkipBlendsForBlendSpaces)
				{
					IBlendStack::FGraphRequestPtr ActiveGraphRequest;
					BlendStackBinding.GetActiveGraph(Context, ActiveGraphRequest);
					if (ActiveGraphRequest && (ActiveGraphRequest->FactoryObject == AnimationAsset))
					{
						check(ActiveGraphRequest->AnimationGraph);
						FAnimNextGraphInstance* ActiveGraphInstance = nullptr;
						BlendStackBinding.GetActiveGraphInstance(Context, ActiveGraphInstance);
						check(ActiveGraphInstance);

						ActiveGraphInstance->AccessVariablesStruct<FBlendSpacePlayerData>([&SharedData, &Binding, &bSkipBlend, &MotionMatchingState, DesiredPlayRate](FBlendSpacePlayerData& InBlendSpacePlayer)
						{
							const float MaxDeltaAssetTimeToTrySkipBlendsForBlendSpaces = SharedData->GetMaxDeltaAssetTimeToTrySkipBlendsForBlendSpaces(Binding);
							if (FMath::Abs(InBlendSpacePlayer.StartPosition - MotionMatchingState.SearchResult.SelectedTime) < MaxDeltaAssetTimeToTrySkipBlendsForBlendSpaces)
							{
								// If we're still on the same blend space, and we're updating blend space inputs dynamically, then DON'T BLEND.
								bSkipBlend = true;

								InBlendSpacePlayer.PlayRate = DesiredPlayRate;

								InBlendSpacePlayer.XAxisSamplePoint = MotionMatchingState.SearchResult.BlendParameters.X;
								InBlendSpacePlayer.YAxisSamplePoint = MotionMatchingState.SearchResult.BlendParameters.Y;
							}
						});
					}
				}

				if (!bSkipBlend)
				{
					FGraphAssetHandle AssetHandle = FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(AnimationAsset);
					FAnimNextFactoryParams FactoryParams = FAnimGraphFactory::GetDefaultParamsForAsset(AssetHandle);

					// currently the default params for sequences and blend spaces lack the FEvaluationNotifiesTrait, necessary trait to be able to evaluate 
					// FEvaluationNotify_BaseInstance(s) during trait post evaluation task (FAnimNextEvaluationNotifiesTask::Execute).
					// Since it's required for MM interaction warp aligment, and there's no data driven way to do so, we have to hard code it here for now.
					FactoryParams.AddTraitStruct<FEvaluationNotifiesTraitSharedData>(ETraitVariableMapping::None, 0);

					// This code manually creates the trait data required for blend space and anim sequence
					// We could utilize UAFAssetData if we add all the options we need to it
					
					if (bIsBlendSpace)
					{
						FactoryParams.AddTraitStruct<FSynchronizeUsingGroupsData>(ETraitVariableMapping::All, 0);
						FactoryParams.AddTraitStruct<FBlendSpacePlayerData>(ETraitVariableMapping::All, 0);

						const EAnimGroupSynchronizationMode SyncMode = SharedData->GetSyncMode(Binding);
						const FVector BlendParameters = MotionMatchingState.SearchResult.BlendParameters;
						const float SelectedTime = MotionMatchingState.SearchResult.SelectedTime;
						const EAnimAssetLoopMode LoopMode = MotionMatchingState.SearchResult.bLoop ? EAnimAssetLoopMode::ForceLoop : EAnimAssetLoopMode::ForceNonLoop;
						
						FactoryParams.AddInitializeTask([SyncMode, BlendParameters, AnimationAsset, DesiredPlayRate, SelectedTime, LoopMode](const FInstanceTaskContext& InInstance)
						{
							InInstance.AccessVariablesStruct<FSynchronizeUsingGroupsData>([SyncMode, BlendParameters](FSynchronizeUsingGroupsData& SynchronizeUsingGroups)
							{
								// Tell blendspace samples to synchronize among themselves, but not with anything else.
								SynchronizeUsingGroups.GroupName = NAME_None;
								SynchronizeUsingGroups.GroupRole = EAnimGroupSynchronizationRole::AlwaysFollower;
								SynchronizeUsingGroups.SyncMode = SyncMode;
								SynchronizeUsingGroups.bMatchSyncPoint = true;
							});

							InInstance.AccessVariablesStruct<FBlendSpacePlayerData>([AnimationAsset, BlendParameters, DesiredPlayRate, SelectedTime, LoopMode](FBlendSpacePlayerData& BlendSpacePlayer)
							{
								BlendSpacePlayer.BlendSpace = Cast<UBlendSpace>(AnimationAsset);

								BlendSpacePlayer.XAxisSamplePoint = BlendParameters.X;
								BlendSpacePlayer.YAxisSamplePoint = BlendParameters.Y;

								BlendSpacePlayer.PlayRate = DesiredPlayRate;
								BlendSpacePlayer.StartPosition = SelectedTime;
								BlendSpacePlayer.LoopMode = LoopMode;
							});
						});
					}
					else
					{
						FactoryParams.AddTraitStruct<FSequencePlayerData>(ETraitVariableMapping::All, 0);

						const float SelectedTime = MotionMatchingState.SearchResult.SelectedTime;
						const EAnimAssetLoopMode LoopMode = MotionMatchingState.SearchResult.bLoop ? EAnimAssetLoopMode::ForceLoop : EAnimAssetLoopMode::ForceNonLoop;

						FactoryParams.AddInitializeTask([SelectedTime, LoopMode, AnimationAsset, DesiredPlayRate](const FInstanceTaskContext& InInstance)
						{
							InInstance.AccessVariablesStruct<FSequencePlayerData>([SelectedTime, LoopMode, AnimationAsset, DesiredPlayRate](FSequencePlayerData& AnimSequencePlayer)
							{
								AnimSequencePlayer.AnimSequence = Cast<UAnimSequence>(AnimationAsset);
								AnimSequencePlayer.PlayRate = DesiredPlayRate;
								AnimSequencePlayer.StartPosition = SelectedTime;
								AnimSequencePlayer.LoopMode = LoopMode;
							});
						});
					}

					const UUAFAnimGraph* AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, AssetHandle, FactoryParams);
					if (AnimationGraph != nullptr)
					{
						IBlendStack::FGraphRequest NewGraphRequest;
						NewGraphRequest.FactoryParams = MoveTemp(FactoryParams);
						NewGraphRequest.BlendArgs = SharedData->GetBlendArguments(Binding);
						NewGraphRequest.AnimationGraph = AnimationGraph;
						NewGraphRequest.FactoryObject = AnimationAsset;
						NewGraphRequest.BlendProfile = SharedData->GetBlendProfile(Binding);

						BlendStackBinding.PushGraph(Context, MoveTemp(NewGraphRequest));
					}
				}
			}
		}

		// Sync latest motion matching state to current graph
		FAnimNextGraphInstance* ActiveGraphInstance = nullptr;
		BlendStackBinding.GetActiveGraphInstance(Context, ActiveGraphInstance);

		// Only override current graph's inputs if we have a valid search result.
		if (ActiveGraphInstance && MotionMatchingState.SearchResult.SelectedAnim)
		{
			ActiveGraphInstance->AccessVariablesStruct<FSequencePlayerData>([DesiredPlayRate](FSequencePlayerData& InSequencePlayer)
			{
				InSequencePlayer.PlayRate = DesiredPlayRate;
			});

			ActiveGraphInstance->AccessVariablesStruct<FBlendSpacePlayerData>([&SharedData, &Binding, DesiredPlayRate](FBlendSpacePlayerData& InBlendSpacePlayer)
			{
				InBlendSpacePlayer.PlayRate = DesiredPlayRate;
				// always updating the blend parameters if bUpdateBlendSpaceInputs
				if (SharedData->GetbUpdateBlendSpaceInputs(Binding))
				{
					InBlendSpacePlayer.XAxisSamplePoint = SharedData->GetXAxisSamplePoint(Binding);
					InBlendSpacePlayer.YAxisSamplePoint = SharedData->GetYAxisSamplePoint(Binding);
				}
			});
		}

		PublishResults(Binding);

		if (MotionMatchingState.SearchResult.bIsInteraction && SharedData->GetbEnableWarp(Binding))
		{
			if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(MotionMatchingState.SearchResult.SelectedAnim))
			{
				const int32 CurrentResultRoleIndex = GetRoleIndex(MultiAnimAsset, MotionMatchingState.SearchResult.Role);
				if (CurrentResultRoleIndex != INDEX_NONE &&
					MotionMatchingState.SearchResult.ActorRootTransforms.Num() == MotionMatchingState.SearchResult.AnimContexts.Num() &&
					MotionMatchingState.SearchResult.ActorRootBoneTransforms.Num() == MotionMatchingState.SearchResult.AnimContexts.Num())
				{
					const int32 NumRoles = MultiAnimAsset->GetNumRoles();
					TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> FullAlignedTransforms;
					FullAlignedTransforms.SetNum(NumRoles);

					CalculateFullAlignedTransforms(MotionMatchingState.SearchResult, MultiAnimAsset->GetPlayLength(FVector::ZeroVector), 0.f, SharedData->GetbWarpUsingRootBone(Binding), FullAlignedTransforms);
					
					FUAFAssetInstance& Instance = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
					FName AlignmentTransformName = SharedData->GetAlignmentTransformVariableName(Binding);
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Instance.SetVariable(FAnimNextVariableReference(AlignmentTransformName), FullAlignedTransforms[CurrentResultRoleIndex]);
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		}

		// Motion Matching must finish updating before smoothers, otherwise we may request a blend after the smoother is done normalizing weights.
		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FMotionMatchingTrait::PublishResults(const TTraitBinding<IUpdate>& Binding) const
	{
		using namespace UE::PoseSearch;

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const FName ResultOutputVariableName = SharedData->GetMotionMatchingResultVariableName(Binding);
		const FName ResultOutputVariableNameAlt = SharedData->GetMotionMatchingResultVariableNameAlt(Binding);

		if (ResultOutputVariableName == NAME_None && ResultOutputVariableNameAlt == NAME_None)
		{
			return;
		}

		const FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		const FMotionMatchingState& MotionMatchingState = InstanceData->MotionMatchingState;
		const UPoseSearchDatabase* CurrentResultDatabase = MotionMatchingState.SearchResult.SelectedDatabase.Get();
		if (CurrentResultDatabase == nullptr || CurrentResultDatabase->Schema == nullptr)
		{
			// @todo: should we log a warning?
		}
#if WITH_EDITOR
		else if (EAsyncBuildIndexResult::Success != FAsyncPoseSearchDatabasesManagement::RequestAsyncBuildIndex(CurrentResultDatabase, ERequestAsyncBuildFlag::ContinueRequest))
		{
			UE_LOGF(LogPoseSearch, Warning, "FMotionMatchingTrait::PreUpdate called while '%ls' is indexing. returning an invalid result..", *CurrentResultDatabase->GetName());
		}
#endif // WITH_EDITOR
		
		const auto PublishResult = [](FAnimNextModuleInstance* ModuleInstance, const FUAFWeakSystemReference& SystemReference, const FPoseSearchBlueprintResult& SearchResult, FName ResultOutputVariableName)
			{
				if (ResultOutputVariableName != NAME_None)
				{
					if (SystemReference.IsValid())
					{
						// storing SearchResult TObjectPtr(s) as TWeakObjectPtr(s) to avoid referencing deallocated memory 
						// in case the module containing this trait gets destroyed before the QueueTask lambda execution
						TWeakObjectPtr<UObject> SelectedAnim = SearchResult.SelectedAnim;
						float SelectedTime = SearchResult.SelectedTime;
						bool bIsContinuingPoseSearch = SearchResult.bIsContinuingPoseSearch;
						float WantedPlayRate = SearchResult.WantedPlayRate;
						bool bLoop = SearchResult.bLoop;
						bool bIsMirrored = SearchResult.bIsMirrored;
						FVector BlendParameters = SearchResult.BlendParameters;
						TWeakObjectPtr<const UPoseSearchDatabase> SelectedDatabase = SearchResult.SelectedDatabase;
						float SearchCost = SearchResult.SearchCost;
						bool bIsInteraction = SearchResult.bIsInteraction;
						FName Role = SearchResult.Role;
						TArray<FTransform> ActorRootTransforms = SearchResult.ActorRootTransforms;
						TArray<FTransform> ActorRootBoneTransforms = SearchResult.ActorRootBoneTransforms;

						SystemReference.QueueTask(
							FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName,
							[SelectedAnim, SelectedTime, bIsContinuingPoseSearch, WantedPlayRate, bLoop,
							bIsMirrored, BlendParameters, SelectedDatabase, SearchCost, bIsInteraction,
							Role, ActorRootTransforms, ActorRootBoneTransforms, ResultOutputVariableName](const UE::UAF::FModuleTaskContext& TaskContext)
							{
								FAnimNextModuleInstance* OtherModuleInstance = TaskContext.GetModuleInstance();
								if (OtherModuleInstance)
								{
									// reconstructing the FPoseSearchBlueprintResult from the input properties
									FPoseSearchBlueprintResult SearchResult;
									SearchResult.SelectedAnim = SelectedAnim.Get();
									SearchResult.SelectedTime = SelectedTime;
									SearchResult.bIsContinuingPoseSearch = bIsContinuingPoseSearch;
									SearchResult.WantedPlayRate = WantedPlayRate;
									SearchResult.bLoop = bLoop;
									SearchResult.bIsMirrored = bIsMirrored;
									SearchResult.BlendParameters = BlendParameters;
									SearchResult.SelectedDatabase = SelectedDatabase.Get();
									SearchResult.SearchCost = SearchCost;
									SearchResult.bIsInteraction = bIsInteraction;
									SearchResult.Role = Role;
									SearchResult.ActorRootTransforms = ActorRootTransforms;
									SearchResult.ActorRootBoneTransforms = ActorRootBoneTransforms;
									PRAGMA_DISABLE_DEPRECATION_WARNINGS
									OtherModuleInstance->SetVariable(FAnimNextVariableReference(ResultOutputVariableName), SearchResult);
									PRAGMA_ENABLE_DEPRECATION_WARNINGS
								}
							},
							UE::UAF::ETaskRunLocation::Before);
					}
					else
					{
						FPoseSearchBlueprintResult SearchResultCopy = SearchResult;
						SearchResultCopy.AnimContexts.Reset();
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						ModuleInstance->SetVariable(FAnimNextVariableReference(ResultOutputVariableName), SearchResultCopy);
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				}
			};

		FAnimNextModuleInstance* ModuleInstance = Binding.GetTraitPtr().GetNodeInstance()->GetOwner().GetModuleInstance();
		check(ModuleInstance);

		const FUAFWeakSystemReference ResultSystemReference = SharedData->GetMotionMatchingResultSystemReference(Binding);
		PublishResult(ModuleInstance, ResultSystemReference, MotionMatchingState.SearchResult, ResultOutputVariableName);
		
		const FUAFWeakSystemReference ResultSystemReferenceAlt = SharedData->GetMotionMatchingResultSystemReferenceAlt(Binding);
		PublishResult(ModuleInstance, ResultSystemReferenceAlt, MotionMatchingState.SearchResult, ResultOutputVariableNameAlt);
	}

	void FMotionMatchingTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		using namespace UE::PoseSearch;

		IEvaluate::PostEvaluate(Context, Binding);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

#if WITH_EDITOR
		if (InstanceData->bIsPostEvaluateBeingCalled)
		{
			UE_LOGF(LogPoseSearch, Error, "FMotionMatchingTrait::PostEvaluate, PostEvaluate called without calling PreUpdate on this frame! Some trait in the TraitStack doesn't propagate correctly the PreUpdate!");
		}
		else
		{
			InstanceData->bIsPostEvaluateBeingCalled = true;
		}
#endif // WITH_EDITOR

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);
		if (SharedData->GetbEnableWarp(Binding))
		{
			// if it's not a multi character interaction we can skip the warping logic entirely
			const FPoseSearchBlueprintResult& SearchResult = InstanceData->MotionMatchingState.SearchResult;
			if (SearchResult.bIsInteraction)
			{
				if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(SearchResult.SelectedAnim))
				{
					const int32 CurrentResultRoleIndex = GetRoleIndex(MultiAnimAsset, SearchResult.Role);
					if (CurrentResultRoleIndex != INDEX_NONE)
					{
						// @todo: WIP hacky, non thread safe (unless proper tick dependencies are in place) way to retrieve the mesh transform until we find a better way
						if (const FAnimNextModuleInstance* ModuleInstance = Context.GetRootGraphInstance().GetModuleInstance())
						{
							if (const UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(ModuleInstance->GetObject()))
							{
								const AActor* Actor = AnimNextComponent->GetOwner();
								check(Actor);
								if (const USkeletalMeshComponent* SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>())
								{
									FAnimNextMotionMatchingTask Task;
									Task.ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
									Task.InstanceData = InstanceData;
									Task.CurrentResultRoleIndex = CurrentResultRoleIndex;
									Task.bWarpUsingRootBone = SharedData->GetbWarpUsingRootBone(Binding);
									Task.WarpingRotationRatio = SharedData->GetWarpingRotationRatio(Binding);
									Task.WarpingTranslationRatio = SharedData->GetWarpingTranslationRatio(Binding);
									Task.WarpingRotationCurveName = SharedData->GetWarpingRotationCurveName(Binding);
									Task.WarpingTranslationCurveName = SharedData->GetWarpingTranslationCurveName(Binding);
#if ENABLE_ANIM_DEBUG
									// Debug Object for VisualLogger
									Task.HostObject = Context.GetHostObject();
#endif // ENABLE_ANIM_DEBUG 
									Context.AppendTask(Task);
								}
							}
						}
					}
				}
			}
		}
	}

	void FMotionMatchingTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		FPoseSearchBlueprintResult& SearchResult = InstanceData->MotionMatchingState.SearchResult;

		Collector.AddReferencedObject(SearchResult.SelectedAnim);
		Collector.AddReferencedObject(SearchResult.SelectedDatabase);

		for (TObjectPtr<const UObject>& AnimContexts : SearchResult.AnimContexts)
		{
			Collector.AddReferencedObject(AnimContexts);
		}
	}

} // UE::UAF

void FAnimNextMotionMatchingTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::PoseSearch;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	check(CurrentResultRoleIndex != INDEX_NONE);

	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	if (!RootMotionProvider)
	{
		UE_LOGF(LogPoseSearch, Error, "FAnimNextMotionMatchingTask::Execute, missing RootMotionProvider");
		return;
	}
	
	const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe)
	{
		return;
	}

	const FPoseSearchBlueprintResult& SearchResult = InstanceData->MotionMatchingState.SearchResult;
	check(SearchResult.bIsInteraction);

	const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(SearchResult.SelectedAnim);
	if (!MultiAnimAsset)
	{
		return;
	}

	const int32 NumRoles = MultiAnimAsset->GetNumRoles();
	if (SearchResult.ActorRootTransforms.Num() != NumRoles)
	{
		// warping is supported only for UMultiAnimAsset(s)
		return;
	}

	FTransform RootMotionDelta = FTransform::Identity;
	if (!RootMotionProvider->ExtractRootMotion(Keyframe->Get()->Attributes, RootMotionDelta))
	{
		return;
	}

	bool OutHasElement = false;
	const float FinalWarpingRotationRatio = FMath::Clamp(Keyframe->Get()->Curves.Get(WarpingRotationCurveName, OutHasElement, WarpingRotationRatio), 0.f, 1.f);
	const float FinalWarpingTranslationRatio = FMath::Clamp(Keyframe->Get()->Curves.Get(WarpingTranslationCurveName, OutHasElement, WarpingTranslationRatio), 0.f, 1.f);

	TArray<FTransform, TInlineAllocator<PreallocatedRolesNum>> FullAlignedTransforms;
	FullAlignedTransforms.SetNum(NumRoles);

	CalculateFullAlignedTransforms(SearchResult, SearchResult.SelectedTime, 0.f, bWarpUsingRootBone, FullAlignedTransforms);

	// @todo: implement this properly once we have UE::AnimationWarping::FRootOffsetProvider
	const FTransform& MeshWithoutOffset = ComponentTransform;
	const FTransform& MeshWithOffset = MeshWithoutOffset;

	// NoTe: keep in mind DeltaAlignment is relative to the previous execution frame so we still need to extract and and apply the current animation root motion transform to get to the current frame full aligned transform.
	FTransform DeltaAlignment = CalculateDeltaAlignment(MeshWithoutOffset, MeshWithOffset, FullAlignedTransforms[CurrentResultRoleIndex], FinalWarpingRotationRatio, FinalWarpingTranslationRatio);
	// @TODO: This is a HACK. Since our warp is framerate dependent, if our characters are at different heights we may generate excessive z velocity resulting in pops at high frame rates.
	{
		FVector DeltaTranslation = DeltaAlignment.GetTranslation();
		DeltaTranslation.Z = 0.0f;
		DeltaAlignment.SetTranslation(DeltaTranslation);
	}

	const FTransform DeltaAlignmentWithRootMotion = DeltaAlignment * RootMotionDelta;

	RootMotionProvider->OverrideRootMotion(DeltaAlignmentWithRootMotion, Keyframe->Get()->Attributes);

#if ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("FAnimNextMotionMatchingTask");

		for (int32 Index = 0; Index < NumRoles; ++Index)
		{
			const FTransform& ActorRootTransform = SearchResult.ActorRootTransforms[Index];
			const FTransform& FullAlignedTransform = FullAlignedTransforms[Index];

			UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, FullAlignedTransform.GetLocation(), ActorRootTransform.GetLocation(), FColorList::Orange, 1.f, TEXT(""));
			UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, ActorRootTransform.GetLocation(), ActorRootTransform.GetLocation() + ActorRootTransform.GetRotation().GetForwardVector() * 35, FColorList::LightGrey, 3.f, TEXT(""));
			UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, FullAlignedTransform.GetLocation(), FullAlignedTransform.GetLocation() + FullAlignedTransform.GetRotation().GetForwardVector() * 30, FColorList::Orange, 2.f, TEXT(""));
		}

		UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, MeshWithOffset.GetLocation(), MeshWithOffset.GetLocation() + MeshWithOffset.GetRotation().GetForwardVector() * 35, FColorList::Blue, 3.f, TEXT(""));
		UE_VLOG_SEGMENT_THICK(HostObject, LogName, Display, MeshWithoutOffset.GetLocation(), MeshWithoutOffset.GetLocation() + MeshWithoutOffset.GetRotation().GetForwardVector() * 40, FColorList::Cyan, 4.f, TEXT(""));
	}
#endif // ENABLE_VISUAL_LOG && ENABLE_ANIM_DEBUG
}
