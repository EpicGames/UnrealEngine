// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AlphaBlend.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IGroupSynchronization.h"
#include "TraitInterfaces/IUpdate.h"
#include "MotionMatchingTraitData.h"
#include "MotionMatchingTrait.generated.h"

namespace UE::UAF
{

struct FMotionMatchingTrait : FAdditiveTrait, IUpdate, IEvaluate, IGarbageCollection
{
	DECLARE_ANIM_TRAIT(FMotionMatchingTrait, FAdditiveTrait)

	using FSharedData = FMotionMatchingTraitSharedData;
		
	struct FInstanceData : FTrait::FInstanceData
	{
		FMotionMatchingState MotionMatchingState;

#if WITH_EDITOR
		bool bIsPostEvaluateBeingCalled = true;
#endif // WITH_EDITOR
	};

	// IUpdate impl
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	// IGarbageCollection impl
	virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;

private:
	void PublishResults(const TTraitBinding<IUpdate>& Binding) const;
};

} // namespace UE::UAF

USTRUCT(Experimental)
struct FAnimNextMotionMatchingTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextMotionMatchingTask)

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	FTransform ComponentTransform = FTransform::Identity;
	UE::UAF::FMotionMatchingTrait::FInstanceData* InstanceData = nullptr;
	int32 CurrentResultRoleIndex = INDEX_NONE;
	bool bWarpUsingRootBone = true;
	float WarpingRotationRatio = 1.f;
	float WarpingTranslationRatio = 1.f;
	FName WarpingRotationCurveName;
	FName WarpingTranslationCurveName;

#if ENABLE_ANIM_DEBUG
	// Debug Object for VisualLogger
	const UObject* HostObject = nullptr;
#endif // ENABLE_ANIM_DEBUG 
};