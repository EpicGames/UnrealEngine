// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseHistoryChooserParameter.h"
#include "UAFAssetInstance.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

namespace 
{

template <typename ValueType>
EPropertyBagResult GetVariableValue(const FUAFAssetInstance* Instance, const FAnimNextVariableReference& Variable, ValueType& OutValue)
{
	while(Instance)
	{
		if (Instance->GetVariable(Variable, OutValue) == EPropertyBagResult::Success)
		{
			return EPropertyBagResult::Success;
		}
	
		Instance = Instance->GetHost();
	}

	return EPropertyBagResult::PropertyNotFound;
}
	
	
// Helper function to find the first data interface instance in the context 
FUAFAssetInstance* GetFirstUAFAssetInstance(FChooserEvaluationContext& Context)
{
	for(const FStructView& Param : Context.Params)
	{
		if(Param.GetScriptStruct()->IsChildOf(FUAFAssetInstance::StaticStruct()))
		{
			return Param.GetPtr<FUAFAssetInstance>();
		}
	}

	return nullptr;
}


}

bool FPoseHistoryAnimProperty::GetValue(FChooserEvaluationContext& Context, FPoseHistoryReference& OutResult) const
{
	if(FUAFAssetInstance* Instance = GetFirstUAFAssetInstance(Context))
	{
		return GetVariableValue(Instance, Variable, OutResult) == EPropertyBagResult::Success;
	}
	return false;
}

bool FPoseHistoryAnimProperty::GetValue(FChooserEvaluationContext& Context, FPoseSearchHistory& OutResult) const
{
	if(FUAFAssetInstance* Instance = GetFirstUAFAssetInstance(Context))
	{
		return GetVariableValue(Instance, Variable, OutResult) == EPropertyBagResult::Success;
	}
	return false;
}

#if WITH_EDITORONLY_DATA
bool FPoseHistoryAnimProperty::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	return false;
}

void FPoseHistoryAnimProperty::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Variable = FAnimNextVariableReference(VariableName_DEPRECATED);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}
#endif