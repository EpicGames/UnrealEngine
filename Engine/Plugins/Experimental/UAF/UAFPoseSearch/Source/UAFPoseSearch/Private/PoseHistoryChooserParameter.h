// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IChooserParameterBool.h"
#include "IChooserParameterFloat.h"
#include "IChooserParameterEnum.h"
#include "PoseSearch/Chooser/ChooserParameterPoseHistoryBase.h"
#include "Variables/AnimNextVariableReference.h"
#include "PoseHistoryChooserParameter.generated.h"

USTRUCT(DisplayName = "Pose History Anim Param")
struct FPoseHistoryAnimProperty :  public FChooserParameterPoseHistoryBase
{
	GENERATED_BODY()
public:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName VariableName_DEPRECATED;
#endif

	UPROPERTY(DisplayName = "Variable", EditAnywhere, Category="Variable")
	FAnimNextVariableReference Variable;

	bool GetValue(FChooserEvaluationContext& Context, FPoseHistoryReference& OutResult) const override;
	bool GetValue(FChooserEvaluationContext& Context, FPoseSearchHistory& OutResult) const override;
	virtual bool IsBound() const { return !Variable.IsNone(); }

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
#endif	
};