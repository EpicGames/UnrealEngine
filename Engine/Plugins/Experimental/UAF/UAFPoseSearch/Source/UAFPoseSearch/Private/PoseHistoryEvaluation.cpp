// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseHistoryEvaluation.h"

namespace UE::PoseSearch
{
	const UE::UAF::FEvaluationVMStackName POSEHISTORY_STACK_NAME = FName(TEXT("PoseHistoryStack"));
}