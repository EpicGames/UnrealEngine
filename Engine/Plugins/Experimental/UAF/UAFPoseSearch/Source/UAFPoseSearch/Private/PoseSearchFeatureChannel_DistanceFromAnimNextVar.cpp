// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchFeatureChannel_DistanceFromAnimNextVar.h"
#include "Component/AnimNextComponent.h"
#include "UAFAssetInstance.h"
#include "PoseSearch/PoseSearchContext.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Variables/AnimNextVariableReference.h"


namespace {
// Helper function to find the first data interface instance in the context 
FUAFAssetInstance* GetFirstAnimNextInstance(FChooserEvaluationContext& Context)
{
	for(const FStructView& Param : Context.Params)
	{
		if(Param.GetScriptStruct() == FUAFAssetInstance::StaticStruct())
		{
			return Param.GetPtr<FUAFAssetInstance>();
		}
	}

	return nullptr;
}
}


void UPoseSearchFeatureChannel_DistanceFromAnimNextVar::BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const
{
	using namespace UE::PoseSearch;
	FChooserEvaluationContext* Context = SearchContext.GetContext(SampleRole);
	
	float Distance = 0.0f;

	if (FUAFAssetInstance* Instance = GetFirstAnimNextInstance(*Context))
	{
		Instance->GetVariable(DistanceVariable, Distance);
	}
	
	FFeatureVectorHelper::EncodeFloat(SearchContext.EditFeatureVector(), ChannelDataOffset, Distance);
}

void UPoseSearchFeatureChannel_DistanceFromAnimNextVar::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DistanceVariable = FAnimNextVariableReference(DistanceVariableName_DEPRECATED);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
}