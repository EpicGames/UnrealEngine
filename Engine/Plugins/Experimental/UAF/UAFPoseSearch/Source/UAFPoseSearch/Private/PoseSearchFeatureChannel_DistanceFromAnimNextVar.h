// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchFeatureChannel_Distance.h"
#include "Variables/AnimNextVariableReference.h"
#include "PoseSearchFeatureChannel_DistanceFromAnimNextVar.generated.h"

UCLASS(Experimental, EditInlineNew, Blueprintable, meta = (DisplayName = "Distance Channel (AnimNext Variable)"), CollapseCategories)
class UPoseSearchFeatureChannel_DistanceFromAnimNextVar : public UPoseSearchFeatureChannel_Distance
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName DistanceVariableName_DEPRECATED;
#endif

	UPROPERTY(DisplayName = "Variable", EditAnywhere, Category="Variable")
	FAnimNextVariableReference DistanceVariable;

	virtual void BuildQuery(UE::PoseSearch::FSearchContext& SearchContext) const override;
	virtual void Serialize(FArchive& Ar) override;
};
