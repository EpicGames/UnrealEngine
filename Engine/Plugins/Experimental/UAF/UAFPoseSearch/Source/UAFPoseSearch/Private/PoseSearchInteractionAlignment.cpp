// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchInteractionAlignment.h"
#include "Animation/AnimRootMotionProvider.h"
#include "EvaluationVM/EvaluationVM.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchInteractionSubsystem.h"
#include "PoseSearch/PoseSearchInteractionUtils.h"
#include "VisualLogger/VisualLogger.h"

void FEvaluationNotify_PoseSearchInteractionAlignment::Start()
{
	WarpStart = FTransform::Identity;
	WarpEnd = FTransform::Identity;
	WarpEndTime = 0.f;
	WarpCurrentTime = 0.f;
	bFirstFrame = true;
}

void FEvaluationNotify_PoseSearchInteractionAlignment::Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData, UE::UAF::FEvaluationVM& VM)
{
	using namespace UE::Anim;
	using namespace UE::PoseSearch;
	using namespace UE::UAF;

	if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
	{
		const UObject* AnimContext = InstanceData.HostObject;

		const IAnimRootMotionProvider* RootMotionProvider = IAnimRootMotionProvider::Get();
		UPoseSearchInteractionSubsystem* InteractionSubsystem = UPoseSearchInteractionSubsystem::GetSubsystem_AnyThread(AnimContext);

		if (RootMotionProvider && InteractionSubsystem)
		{
			// thread safety note: because multi character motion matching adds a tick dependecy between character and UAF,
			// using GetContextTransform(AnimContext, false) is thread safe
			const FTransform ContextTransform = GetContextTransform(AnimContext, false);

			if (bFirstFrame)
			{
				FPoseSearchBlueprintResult Result;
				InteractionSubsystem->GetResult_AnyThread(AnimContext, Result);

				if (Result.SelectedAnim)
				{
					check(Result.RoleIndex >= 0 && Result.RoleIndex < Result.AnimContexts.Num());

					WarpStart = ContextTransform;

					WarpEndTime = EndTime - CurrentTime;

					const float TimeOffset = EndTime - Result.SelectedTime;

					// @todo: this should be a property / parameter
					static bool bWarpUsingRootBone = false;

					TArray<FTransform, TInlineAllocator<UE::PoseSearch::PreallocatedRolesNum>> FullAlignedTransforms;
					FullAlignedTransforms.SetNum(Result.AnimContexts.Num());
					UE::PoseSearch::CalculateFullAlignedTransforms(Result, Result.SelectedTime, TimeOffset, bWarpUsingRootBone, FullAlignedTransforms);

					WarpEnd = FullAlignedTransforms[Result.RoleIndex];
				}
			}

			if (WarpEndTime > UE_KINDA_SMALL_NUMBER)
			{
				const float Alpha = FMath::Min(WarpCurrentTime / WarpEndTime, 1.f);
				FTransform WarpCurrent;
				WarpCurrent.Blend(WarpStart, WarpEnd, Alpha);

				const FTransform RootMotion = WarpCurrent.GetRelativeTransform(ContextTransform);
				RootMotionProvider->OverrideRootMotion(RootMotion, Keyframe->Get()->Attributes);

				UE_VLOG_SEGMENT(AnimContext, "MMIAlignment", Display, WarpStart.GetLocation(), WarpStart.GetLocation() + WarpStart.GetRotation().RotateVector(FVector::XAxisVector) * 30.f, FColor::Red, TEXT(""));
				UE_VLOG_SEGMENT(AnimContext, "MMIAlignment", Display, WarpCurrent.GetLocation(), WarpCurrent.GetLocation() + WarpCurrent.GetRotation().RotateVector(FVector::XAxisVector) * 30.f, FColor::Green, TEXT(""));
				UE_VLOG_SEGMENT(AnimContext, "MMIAlignment", Display, WarpEnd.GetLocation(), WarpEnd.GetLocation() + WarpEnd.GetRotation().RotateVector(FVector::XAxisVector) * 30.f, FColor::Blue, TEXT(""));
			}
		}
	}

	WarpCurrentTime += InstanceData.DeltaTime;
	bFirstFrame = false;
}
