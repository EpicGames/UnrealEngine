// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNext/EvaluationNotifiesTrait.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "PoseSearchInteractionAlignment.generated.h"

USTRUCT(Experimental)
struct FEvaluationNotify_PoseSearchInteractionAlignment : public FEvaluationNotify_BaseInstance
{
	GENERATED_BODY()

	virtual void Start() override;
	virtual void Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData, UE::UAF::FEvaluationVM& VM) override;

	FTransform WarpStart = FTransform::Identity;
	FTransform WarpEnd = FTransform::Identity;
	float WarpEndTime = 0.f;
	float WarpCurrentTime = 0.f;
	bool bFirstFrame = true;
};

UCLASS(Experimental, MinimalAPI, BlueprintType, DisplayName="MMI Alignment")
class UNotifyState_PoseSearchInteractionAlignment : public UAnimNotifyState
{
	GENERATED_BODY()
};