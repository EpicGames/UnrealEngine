// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchResultEmulatorTrait.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Traits/SequencePlayerTraitData.h"
#include "Traits/BlendSpacePlayerTraitData.h"
#include "PoseSearch/MultiAnimAsset.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/IGraphFactory.h"
#include "TraitInterfaces/ITimeline.h"
#include "EvaluationVM/EvaluationVM.h"
#include "Factory/AnimGraphFactory.h"
#include "Traits/SynchronizeUsingGroupsTraitData.h"
#include "UAF/UAFAssetFactory.h"

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FPoseSearchResultEmulatorTrait)

#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FPoseSearchResultEmulatorTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
#undef TRAIT_INTERFACE_ENUMERATOR

	void FPoseSearchResultEmulatorTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		using namespace UE::PoseSearch;
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();

		TTraitBinding<IBlendStack> BlendStackBinding;
		if (!Binding.GetStackInterface(BlendStackBinding))
		{
			UE_LOGF(LogPoseSearch, Error, "FPoseSearchResultEmulatorTrait::PreUpdate, missing IBlendStack");
			return;
		}

		TTraitBinding<ITimeline> TimelineTrait;
		if (!Binding.GetStackInterface(TimelineTrait))
		{
			UE_LOGF(LogPoseSearch, Error, "FPoseSearchResultEmulatorTrait::PreUpdate, missing ITimeline");
			return;
		}

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const UObject* AnimObject = SharedData->GetSelectedAnim(Binding);
		const FPoseSearchBlueprintResult& Result = SharedData->GetPoseSearchResult(Binding);
		if (AnimObject == nullptr)
		{
			AnimObject = Result.SelectedAnim.Get();
		}

		if (AnimObject)
		{
			// Shared parameters
			const FName Role = SharedData->GetRole(Binding);
			const float SelectedTime = SharedData->GetSelectedTime(Binding);
			const float WantedPlayRate = SharedData->GetWantedPlayRate(Binding);
			const bool bLoop = SharedData->GetbLoop(Binding);
			const float XAxisSamplePoint = SharedData->GetXAxisSamplePoint(Binding);
			const float YAxisSamplePoint = SharedData->GetYAxisSamplePoint(Binding);

			const UAnimationAsset* AnimationAsset = nullptr;
			if (const UMultiAnimAsset* MultiAnimAsset = Cast<UMultiAnimAsset>(AnimObject))
			{
				AnimationAsset = MultiAnimAsset->GetAnimationAsset(Role);
			}
			else
			{
				AnimationAsset = Cast<UAnimationAsset>(AnimObject);
			}

			IBlendStack::FGraphRequestPtr ActiveGraphRequest;
			BlendStackBinding.GetActiveGraph(Context, ActiveGraphRequest);
			const UObject* CurrentAnim = nullptr;
			float TimeDelta = 0.0f;
			bool bIsBlendSpace = AnimationAsset && AnimationAsset->IsA<UBlendSpace>();
			const bool bHasValidActiveGraph = ActiveGraphRequest && ActiveGraphRequest->FactoryObject;
			if (bHasValidActiveGraph)
			{
				CurrentAnim = ActiveGraphRequest->FactoryObject;
				FTimelineState TimelineState;
				if(TimelineTrait.GetState(Context, TimelineState))
				{
					float TimelinePosition = TimelineState.GetPosition();

					const bool bCurrentAnimIsBlendSpace = CurrentAnim->IsA<UBlendSpace>();
					if (bCurrentAnimIsBlendSpace)
					{
						// Convert to normalized time.
						const float TimelineDuration = TimelineState.GetDuration();
						if (TimelineDuration > 0.0f)
						{
							// NOTE: This doesn't work because AnimNext normalized time does not match database indexing normalized time.
							TimelinePosition /= TimelineState.GetDuration();
						}
					}

					TimeDelta = FMath::Abs(TimelinePosition - SelectedTime);
				}
			}

			const float MaxTimeDeltaAllowed = SharedData->GetMaxTimeDeltaAllowed(Binding);
			const bool bDoBlend =	CurrentAnim != AnimationAsset
								||	TimeDelta > FMath::Max(MaxTimeDeltaAllowed, UE_SMALL_NUMBER)
								// If it's not a continuing pose search, then our search requested a new pose.
								|| !Result.bIsContinuingPoseSearch;

			if (bDoBlend && AnimationAsset)
			{
				TInstancedStruct<FUAFGraphFactoryAsset> AssetHandle = FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(AnimationAsset);
				FAnimNextFactoryParams FactoryParams = FAnimGraphFactory::GetDefaultParamsForAsset(AssetHandle);
				if (bIsBlendSpace)
				{
					FactoryParams.AddTraitStruct<FSynchronizeUsingGroupsData>(ETraitVariableMapping::All, 0);
					FactoryParams.AddTraitStruct<FBlendSpacePlayerData>(ETraitVariableMapping::All, 0);

					const EAnimAssetLoopMode LoopMode = bLoop ? EAnimAssetLoopMode::ForceLoop : EAnimAssetLoopMode::ForceNonLoop;

					FactoryParams.AddInitializeTask([XAxisSamplePoint, YAxisSamplePoint, AnimationAsset, WantedPlayRate, SelectedTime, LoopMode](const FInstanceTaskContext& InInstance)
					{
						InInstance.AccessVariablesStruct<FSynchronizeUsingGroupsData>([](FSynchronizeUsingGroupsData& SynchronizeUsingGroups)
						{
							// Tell blendspace samples to synchronize among themselves, but not with anything else.
							SynchronizeUsingGroups.GroupName = NAME_None;
							SynchronizeUsingGroups.GroupRole = EAnimGroupSynchronizationRole::AlwaysFollower;
							SynchronizeUsingGroups.SyncMode = EAnimGroupSynchronizationMode::SynchronizeUsingUniqueGroupName;
							SynchronizeUsingGroups.bMatchSyncPoint = true;
						});

						InInstance.AccessVariablesStruct<FBlendSpacePlayerData>([AnimationAsset, XAxisSamplePoint, YAxisSamplePoint, WantedPlayRate, SelectedTime, LoopMode](FBlendSpacePlayerData& BlendSpacePlayer)
						{
							BlendSpacePlayer.BlendSpace = Cast<UBlendSpace>(AnimationAsset);
							BlendSpacePlayer.XAxisSamplePoint = XAxisSamplePoint;
							BlendSpacePlayer.YAxisSamplePoint = YAxisSamplePoint;
							BlendSpacePlayer.PlayRate = WantedPlayRate;
							BlendSpacePlayer.StartPosition =  SelectedTime;
							BlendSpacePlayer.LoopMode = LoopMode;
						});
					});
				}
				else
				{
					const EAnimAssetLoopMode LoopMode = bLoop ? EAnimAssetLoopMode::ForceLoop : EAnimAssetLoopMode::ForceNonLoop;
	
					FactoryParams.AddInitializeTask([SelectedTime, LoopMode, AnimationAsset, WantedPlayRate](const FInstanceTaskContext& InInstance)
					{
						InInstance.AccessVariablesStruct<FSequencePlayerData>([SelectedTime, LoopMode, AnimationAsset, WantedPlayRate](FSequencePlayerData& AnimSequencePlayer)
						{
							AnimSequencePlayer.AnimSequence = Cast<UAnimSequence>(AnimationAsset);
							AnimSequencePlayer.PlayRate = WantedPlayRate;
							AnimSequencePlayer.StartPosition = SelectedTime;
							AnimSequencePlayer.LoopMode = LoopMode;
						});
					});
				}

				const UUAFAnimGraph* AnimationGraph = IGraphFactory::GetOrBuildGraph(Context, Binding, AssetHandle, FactoryParams);
				if (AnimationGraph != nullptr)
				{
					IBlendStack::FGraphRequest NewGraphRequest;
					NewGraphRequest.BlendArgs = SharedData->GetBlendArguments(Binding);
					NewGraphRequest.AnimationGraph = AnimationGraph;
					NewGraphRequest.FactoryObject = AnimationAsset;
					NewGraphRequest.FactoryParams = MoveTemp(FactoryParams);
					BlendStackBinding.PushGraph(Context, MoveTemp(NewGraphRequest));
				}
			}

			// Update the active graph instance according to continuing parameters
			FAnimNextGraphInstance* ActiveGraphInstance = nullptr;
			BlendStackBinding.GetActiveGraphInstance(Context, ActiveGraphInstance);
			if (ActiveGraphInstance)
			{
				ActiveGraphInstance->AccessVariablesStruct<FSequencePlayerData>([WantedPlayRate](FSequencePlayerData& InAnimSequencePlayer)
				{
					InAnimSequencePlayer.PlayRate = WantedPlayRate;
				});

				ActiveGraphInstance->AccessVariablesStruct<FBlendSpacePlayerData>([XAxisSamplePoint, YAxisSamplePoint, WantedPlayRate](FBlendSpacePlayerData& InBlendSpacePlayer)
				{
					InBlendSpacePlayer.PlayRate = WantedPlayRate;
					InBlendSpacePlayer.XAxisSamplePoint = XAxisSamplePoint;
					InBlendSpacePlayer.YAxisSamplePoint = YAxisSamplePoint;
				});
			}
		}

		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FPoseSearchResultEmulatorTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		const bool bDisableRootMotion = SharedData->GetbDisableRootMotion(Binding);

		if (bDisableRootMotion)
		{
			FAnimNextPoseSearchResultEmulatorTask Task;
			Context.AppendTask(Task);
		}
	}
	
} // UE::UAF

void FAnimNextPoseSearchResultEmulatorTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		return;
	}

	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	if (!RootMotionProvider)
	{
		UE_LOGF(LogPoseSearch, Error, "FAnimNextPoseSearchResultEmulatorTask::Execute, missing RootMotionProvider");
		return;
	}

	const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0);
	if (!Keyframe)
	{
		return;
	}

	RootMotionProvider->OverrideRootMotion(FTransform::Identity, Keyframe->Get()->Attributes);
}
