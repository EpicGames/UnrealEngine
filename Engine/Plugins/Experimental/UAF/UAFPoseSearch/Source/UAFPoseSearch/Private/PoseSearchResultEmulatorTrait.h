// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AlphaBlend.h"
#include "CoreMinimal.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IUpdate.h"
#include "PoseSearchResultEmulatorTrait.generated.h"

USTRUCT(meta = (DisplayName = "Pose Search Result Emulator"))
struct FPoseSearchResultEmulatorTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<UObject> SelectedAnim = nullptr;

	// @TODO: This is a super hack, but we don't have a good way to use TWeakObjectPtr pose search result. Do this for now.
	UPROPERTY(EditAnywhere, Category = Default)
	FPoseSearchBlueprintResult PoseSearchResult;
	
	UPROPERTY(EditAnywhere, Category = Default)
	float SelectedTime = 0.f;
	
	UPROPERTY(EditAnywhere, Category = Default)
	float WantedPlayRate = 0.f;

	UPROPERTY(EditAnywhere, Category = Default)
	bool bLoop = false;
	
	UPROPERTY(EditAnywhere, Category = Default)
	float XAxisSamplePoint = 0.0f;

	UPROPERTY(EditAnywhere, Category = Default)
	float YAxisSamplePoint = 0.0f;

	UPROPERTY(EditAnywhere, Category = Default)
	FName Role;

	UPROPERTY(EditAnywhere, Category = Default)
	FAlphaBlendArgs BlendArguments;

	UPROPERTY(EditAnywhere, Category = Default)
	float MaxTimeDeltaAllowed = 0.1f;

	UPROPERTY(EditAnywhere, Category = Default)
	bool bDisableRootMotion = false;

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(SelectedAnim) \
		GeneratorMacro(PoseSearchResult) \
		GeneratorMacro(SelectedTime) \
		GeneratorMacro(WantedPlayRate) \
		GeneratorMacro(bLoop) \
		GeneratorMacro(XAxisSamplePoint) \
		GeneratorMacro(YAxisSamplePoint) \
		GeneratorMacro(Role) \
		GeneratorMacro(BlendArguments) \
		GeneratorMacro(MaxTimeDeltaAllowed) \
		GeneratorMacro(bDisableRootMotion) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FPoseSearchResultEmulatorTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{

struct FPoseSearchResultEmulatorTrait : FAdditiveTrait, IUpdate, IEvaluate
{
	DECLARE_ANIM_TRAIT(FPoseSearchResultEmulatorTrait, FAdditiveTrait)

	using FSharedData = FPoseSearchResultEmulatorTraitSharedData;
		
	struct FInstanceData : FTrait::FInstanceData
	{
	};

	// IUpdate impl
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;
};

} // namespace UE::UAF

USTRUCT(Experimental)
struct FAnimNextPoseSearchResultEmulatorTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextPoseSearchResultEmulatorTask)

	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;
};