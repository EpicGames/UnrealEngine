// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchSteerAlongTrajectory.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Math/SpringMath.h"
#include "Animation/TrajectoryTypes.h"
#include "EvaluationVM/EvaluationVM.h"
#include "PoseSearch/PoseSearchContext.h"
#include "VisualLogger/VisualLogger.h"
#include "Module/AnimNextModuleInstance.h"

namespace UE::UAF::PoseSearch
{
	template<typename ValueType>
	static EPropertyBagResult AccessVariable(const FUAFAssetInstance* Instance, const FAnimNextVariableReference& Variable, TFunctionRef<void(ValueType&)> Function)
	{
		while (Instance)
		{
			if (EPropertyBagResult::Success == Instance->AccessVariable(Variable, Function))
			{
				return EPropertyBagResult::Success;
			}
			Instance = Instance->GetHost();
		}
		return EPropertyBagResult::PropertyNotFound;
	}
} // namespace UE::UAF::PoseSearch

void FEvaluationNotify_PoseSearchSteerAlongTrajectory::Start()
{
	AngularVelocity = FVector::ZeroVector;
}

void FEvaluationNotify_PoseSearchSteerAlongTrajectory::Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData, UE::UAF::FEvaluationVM& VM)
{
	using namespace UE::Anim;
	using namespace UE::PoseSearch;
	using namespace UE::UAF::PoseSearch;
	using namespace UE::UAF;

	if (const TUniquePtr<FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<FKeyframeState>>(KEYFRAME_STACK_NAME, 0))
	{
		if (const IAnimRootMotionProvider* RootMotionProvider = IAnimRootMotionProvider::Get())
		{
			const UNotifyState_PoseSearchSteerAlongTrajectory* SteerAlongTrajectory = CastChecked<UNotifyState_PoseSearchSteerAlongTrajectory>(AnimNotify);

			FVector TrajectoryDirection;
			auto TransformTrajectoryAccessor = [SteerAlongTrajectory, &TrajectoryDirection](FTransformTrajectory& TransformTrajectory) -> void
				{
					const FTransformTrajectorySample A = TransformTrajectory.GetSampleAtTime(SteerAlongTrajectory->StartTrajectorySampleTime, true);
					const FTransformTrajectorySample B = TransformTrajectory.GetSampleAtTime(SteerAlongTrajectory->EndTrajectorySampleTime, true);
					TrajectoryDirection = B.Position - A.Position;
				};

			TFunctionRef<void(FTransformTrajectory&)> TransformTrajectoryAccessorRef = TransformTrajectoryAccessor;
			if (EPropertyBagResult::Success == AccessVariable(InstanceData.Instance, SteerAlongTrajectory->Trajectory, TransformTrajectoryAccessorRef))
			{
				if (TrajectoryDirection.Normalize(UE_KINDA_SMALL_NUMBER))
				{
					FStackAttributeContainer& Attributes = Keyframe->Get()->Attributes;

					const UObject* AnimContext = InstanceData.HostObject;

					// @todo: we can also use TransformTrajectory.GetSampleAtTime(0) to get the ContextTransform
					// thread safety note: because multi character motion matching adds a tick dependecy between character and UAF,
					// using GetContextTransform(AnimContext, false) is thread safe
					const FTransform ContextTransform = GetContextTransform(AnimContext, false);
					
					FTransform RootMotion;
					RootMotionProvider->ExtractRootMotion(Attributes, RootMotion);

					FVector RootMotionDirection = RootMotion.GetTranslation();
					RootMotionDirection = ContextTransform.TransformVectorNoScale(RootMotionDirection);
					if (RootMotionDirection.Normalize(UE_KINDA_SMALL_NUMBER))
					{
						const FQuat RotationDelta = FQuat::FindBetweenVectors(RootMotionDirection, TrajectoryDirection);

						FQuat RootMotionRotation = RootMotion.GetRotation();
						const FQuat RootMotionTargetRotation = RootMotion.GetRotation() * RotationDelta;

						SpringMath::CriticalSpringDamperQuat(RootMotionRotation, AngularVelocity, RootMotionTargetRotation, SteerAlongTrajectory->SmoothingTime, InstanceData.DeltaTime);

						RootMotion.SetRotation(RootMotionRotation);
						RootMotionProvider->OverrideRootMotion(RootMotion, Attributes);

						UE_VLOG_SEGMENT(AnimContext, "MMISteerAlongTrajectory", Display, ContextTransform.GetLocation(), ContextTransform.GetLocation() + TrajectoryDirection * 60.f, FColor::Blue, TEXT(""));
						UE_VLOG_SEGMENT(AnimContext, "MMISteerAlongTrajectory", Display, ContextTransform.GetLocation(), ContextTransform.GetLocation() + RootMotionDirection * 60.f, FColor::Green, TEXT(""));
					}
				}
			}
		}
	}
}