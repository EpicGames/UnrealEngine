// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNext/EvaluationNotifiesTrait.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "PoseSearchSteerAlongTrajectory.generated.h"

USTRUCT(Experimental)
struct FEvaluationNotify_PoseSearchSteerAlongTrajectory : public FEvaluationNotify_BaseInstance
{
	GENERATED_BODY()

	virtual void Start() override;
	virtual void Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData, UE::UAF::FEvaluationVM& VM) override;
	
	FVector AngularVelocity = FVector::ZeroVector;
};

UCLASS(Experimental, MinimalAPI, BlueprintType, DisplayName="MMI Steer Along Trajectory")
class UNotifyState_PoseSearchSteerAlongTrajectory : public UAnimNotifyState
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="Steering")
	FAnimNextVariableReference Trajectory;

	UPROPERTY(EditAnywhere, Category="Steering")
	float StartTrajectorySampleTime = 0.5f;

	UPROPERTY(EditAnywhere, Category="Steering")
	float EndTrajectorySampleTime = 1.f;

	UPROPERTY(EditAnywhere, Category="Steering")
	float SmoothingTime = 1.f;
};