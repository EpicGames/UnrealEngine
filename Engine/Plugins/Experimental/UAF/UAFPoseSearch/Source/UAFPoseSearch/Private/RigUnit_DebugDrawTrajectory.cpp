// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_DebugDrawTrajectory.h"

#include "Animation/TrajectoryTypes.h"
#include "Component/AnimNextComponent.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "Module/AnimNextModuleInstance.h"

FRigUnit_DebugDrawTrajectory_Execute()
{
#if ENABLE_ANIM_DEBUG
	if (!Enabled)
		return;

	using namespace UE::UAF;

	const FUAFAssetContextData& ContextData = ExecuteContext.GetContextData<FUAFAssetContextData>();
	FAnimNextModuleInstance* ModuleInstance = ContextData.GetInstance().GetRootInstance();
	if (ModuleInstance == nullptr)
	{
		return;
	}

	// TODO: this is not thread safe at the moment. Once we get the 'weak semantics' CL checked in, we can move commonly-used debug info
	// into a module component that can be used to access GT data where needed. 
	UUAFComponent* Component = Cast<UUAFComponent>(ModuleInstance->GetObject());

	UTransformTrajectoryBlueprintLibrary::DebugDrawTrajectory(Trajectory, Component, LogPoseSearch, ELogVerbosity::Display, DebugThickness, DebugOffset, DisplayVelocity);
#endif
}
