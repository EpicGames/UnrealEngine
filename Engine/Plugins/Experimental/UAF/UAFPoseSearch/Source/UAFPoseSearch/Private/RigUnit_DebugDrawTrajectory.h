// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Animation/TrajectoryTypes.h"

#include "RigUnit_DebugDrawTrajectory.generated.h"

/*
 * A debug drawing node to draw a transform trajectory
 */
USTRUCT(meta = (DisplayName = "Debug Draw Trajectory", Category = "Trajectory", NodeColor = "0, 1, 1", Keywords = "Mover, Trajectory, Debug"))
struct FRigUnit_DebugDrawTrajectory: public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The trajectory to draw
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Trajectory", meta = (Input, Output))
	FTransformTrajectory Trajectory;

	// The line thickness to use for the debug drawing
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Trajectory", meta = (Input))
	float DebugThickness = 0.f;

	// The transform offset to use for the debug drawing
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Trajectory", meta = (Input))
	float DebugOffset = 0.f;

	// If False the debug drawing will be skipped
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Trajectory", meta = (Input))
	bool Enabled = false;

	// Whether to display the velocity of the samples in on screen text
	UPROPERTY(EditAnywhere, Category = "Debug Draw - Trajectory", meta = (Input))
	bool DisplayVelocity = false;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};
