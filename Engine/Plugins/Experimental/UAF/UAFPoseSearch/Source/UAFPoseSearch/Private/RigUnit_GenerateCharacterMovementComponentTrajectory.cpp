// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_GenerateCharacterMovementComponentTrajectory.h"

#include "GameFramework/CharacterMovementComponent.h"
#include "PoseSearch/PoseSearchDefines.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"
#include "UAFLogging.h"

FRigUnit_GenerateCharacterMovementComponentTrajectory_Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigUnit_GenerateCharacterMovementComponentTrajectory_Execute);
	using namespace UE::UAF;

	if (CharacterMovementComponent == nullptr)
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Failed to get Character Movement Component."));
		return;
	}

	AActor* Owner = CharacterMovementComponent->GetOwner();
	if (Owner == nullptr)
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Failed to get Character Movement Component's actor owner."));
		return;
	}
	
	UPoseSearchTrajectoryLibrary::PoseSearchGenerateTransformTrajectory(
		Owner,
		TrajectoryData,
		DeltaTime,
		InOutTrajectory,
		InOutDesiredControllerYawLastUpdate,
		InOutTrajectory,
		HistorySamplingInterval,
		NumHistorySamples,
		PredictionSamplingInterval,
		NumPredictionSamples
	);
}
