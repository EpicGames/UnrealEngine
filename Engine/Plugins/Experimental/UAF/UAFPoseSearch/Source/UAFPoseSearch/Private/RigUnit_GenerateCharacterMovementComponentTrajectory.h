// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Animation/TrajectoryTypes.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "PoseSearch/PoseSearchTrajectoryLibrary.h"

#include "RigUnit_GenerateCharacterMovementComponentTrajectory.generated.h"

/*
 * A utility node to generate a character movement trajectory from a character movement component
 */
USTRUCT(meta = (DisplayName = "Generate Trajectory from Character Movement Component", Category = "Character Movement", NodeColor = "0, 1, 1", Keywords = "Trajectory"))
struct FRigUnit_GenerateCharacterMovementComponentTrajectory : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The input character movement component to create the trajectory for
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	TObjectPtr<UCharacterMovementComponent> CharacterMovementComponent;

	// This should be the most recent simulation time that was used to get us to our current state
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float DeltaTime = 0.f;

	// The sampling interval to use for the history
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float HistorySamplingInterval = -1.f;

	// The upper limit of samples to use for the history
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	int32 NumHistorySamples = 30;

	// The interval to use for prediction
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	float PredictionSamplingInterval = 0.1f;

	// The upper limit of samples to use for prediction
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	int32 NumPredictionSamples = 15;

	// The input trajectory data to use
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input))
	FPoseSearchTrajectoryData TrajectoryData;

	// The resulting trajectory
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input, Output))
	FTransformTrajectory InOutTrajectory;

	// The resulting yaw at the last update
	UPROPERTY(EditAnywhere, Category = "Trajectory", meta = (Input, Output))
	float InOutDesiredControllerYawLastUpdate = 0.0f;
	
	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};
