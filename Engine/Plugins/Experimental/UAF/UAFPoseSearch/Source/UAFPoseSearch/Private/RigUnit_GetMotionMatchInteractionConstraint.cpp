// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_GetMotionMatchInteractionConstraint.h"
#include "PoseSearch/PoseSearchInteractionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetMotionMatchInteractionConstraint)

FRigUnit_GetMotionMatchInteractionConstraint_Execute()
{
	UPoseSearchInteractionLibrary::GetMotionMatchInteractionConstraint(ExecuteContext.GetOwningObject(), SocketName, DesiredReach, Transform, IsValid, bCompareOwningActors);
}
