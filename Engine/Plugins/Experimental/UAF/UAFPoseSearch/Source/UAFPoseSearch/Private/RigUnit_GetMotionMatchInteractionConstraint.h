// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "RigUnit_GetMotionMatchInteractionConstraint.generated.h"

/** Gets the motion matching interaction constraint property associated to SocketName */
USTRUCT(meta=(DisplayName="Get Motion Match Interaction Constraint", Category="Animation Graph", NodeColor="0, 1, 1"))
struct FRigUnit_GetMotionMatchInteractionConstraint : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// Socket name associated with the constraint
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FName SocketName = NAME_None;

	// search the constraint for the owning actor instead of using the AnimContext
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	bool bCompareOwningActors = false;
	
	// output desired reach for the constraint
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	float DesiredReach = 0.f;
	
	// output transform for the constraint
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	FTransform Transform = FTransform::Identity;
	
	// validity of the requested constraint
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	bool IsValid = false;
};
