// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_MultiAnimGetAnimationAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_MultiAnimGetAnimationAsset)

FRigUnit_MultiAnimGetAnimationAsset_Execute()
{
	if (MultiAnimAsset)
	{
		Result = MultiAnimAsset->GetAnimationAsset(Role);
	}
}
