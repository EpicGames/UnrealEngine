// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "Animation/AnimationAsset.h"
#include "PoseSearch/MultiAnimAsset.h"

#include "RigUnit_MultiAnimGetAnimationAsset.generated.h"

/** Gets anim asset for a particular role from a MultiAnimAsset */
USTRUCT(meta=(DisplayName="Multi Anim Get Animation Asset", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="MultiAnim"))
struct FRigUnit_MultiAnimGetAnimationAsset : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// MultiAnim Asset to search for role on
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<UMultiAnimAsset> MultiAnimAsset = nullptr;

	// Role to search for within the MultiAnimAsset
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FName Role = NAME_None;

	// Result of the role search in the MultiAnimAsset
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	TObjectPtr<UAnimationAsset> Result = nullptr;
};
