// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "RigUnit_PoseSearchDatabaseGetTags.generated.h"

class UPoseSearchDatabase;

/** Get metadata tags for a specified pose search database */
USTRUCT(meta=(DisplayName="Get Database Tags", Category="Pose Search", NodeColor="0, 1, 1", Keywords="PoseSearch"))
struct FRigUnit_PoseSearchDatabaseGetTags : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	// The data base to retrieve the meta data tags from
	UPROPERTY(EditAnywhere, Transient, Category = "Pose Search", meta = (Input))
	TObjectPtr<UPoseSearchDatabase> Database;

	// The resulting array of meta data tags
	UPROPERTY(EditAnywhere, Transient, Category = "Pose Search", meta = (Output))
	TArray<FName> OutTags;
};
