// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_PoseSearchResultGetSelectedDatabase.h"
#include "PoseSearch/PoseSearchDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_PoseSearchResultGetSelectedDatabase)

FRigUnit_PoseSearchResultGetSelectedDatabase_Execute()
{
	if (PoseSearchResult.SelectedDatabase)
	{
		OutDatabase = const_cast<UPoseSearchDatabase*>(PoseSearchResult.SelectedDatabase.Get());
	}
}