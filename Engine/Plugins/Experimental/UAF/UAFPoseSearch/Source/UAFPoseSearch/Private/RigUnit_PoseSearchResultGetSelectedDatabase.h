// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "PoseSearch/PoseSearchResult.h"

#include "RigUnit_PoseSearchResultGetSelectedDatabase.generated.h"

/** Gets selected database from a pose search result */
USTRUCT(meta=(DisplayName="Get Selected Database ", Category="Pose Search", NodeColor="0, 1, 1", Keywords="PoseSearch"))
struct FRigUnit_PoseSearchResultGetSelectedDatabase : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	// Role to search for within the MultiAnimAsset
	UPROPERTY(EditAnywhere, Transient, Category = "Pose Search", meta = (Input))
	FPoseSearchBlueprintResult PoseSearchResult;

	// Result of the role search in the MultiAnimAsset
	UPROPERTY(EditAnywhere, Transient, Category = "Pose Search", meta = (Output))
	TObjectPtr<UPoseSearchDatabase> OutDatabase = nullptr;
};