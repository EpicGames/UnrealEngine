// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/IScopedTraitInterface.h"
#include "TraitCore/TraitBinding.h"

namespace UE::PoseSearch
{
	struct IPoseHistory;
}

namespace UE::UAF
{
	/**
	 * IPoseHistory
	 */
	struct IPoseHistory : IScopedTraitInterface
	{
		DECLARE_ANIM_TRAIT_INTERFACE(IPoseHistory)

		virtual const UE::PoseSearch::IPoseHistory* GetPoseHistory(FExecutionContext& Context, const TTraitBinding<IPoseHistory>& Binding) const;

#if WITH_EDITOR
		virtual const FText& GetDisplayName() const override;
		virtual const FText& GetDisplayShortName() const override;
#endif // WITH_EDITOR
	};

	/**
	 * Specialization for trait binding.
	 */
	template<>
	struct TTraitBinding<IPoseHistory> : FTraitBinding
	{
		const UE::PoseSearch::IPoseHistory* GetPoseHistory(FExecutionContext& Context) const
		{
			return GetInterface()->GetPoseHistory(Context, *this);
		}

	protected:
		const IPoseHistory* GetInterface() const { return GetInterfaceTyped<IPoseHistory>(); }
	};
}
