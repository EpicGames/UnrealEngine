// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseSearch/PoseSearchHistory.h"
#include "EvaluationVM/EvaluationVM.h"

namespace UE::PoseSearch
{
	extern UAFPOSESEARCH_API const UE::UAF::FEvaluationVMStackName POSEHISTORY_STACK_NAME;

	struct FPoseHistoryEvaluationHelper
	{
		TSharedPtr<UE::PoseSearch::FGenerateTrajectoryPoseHistory, ESPMode::ThreadSafe> PoseHistoryPtr = nullptr;
	};
}

namespace UE::UAF
{
	ANIM_NEXT_ENABLE_EVALUATION_STACK_USAGE(TUniquePtr<UE::PoseSearch::FPoseHistoryEvaluationHelper>);
}