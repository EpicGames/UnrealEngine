// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFPoseSearch : ModuleRules
	{
		public UAFPoseSearch(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Chooser",
					"Core",
					"CoreUObject",
					"Engine",
					"EvaluationNotifiesRuntime",
					"HierarchyTableAnimationRuntime",
					"HierarchyTableRuntime",
					"PoseSearch",
					"RigVM",
					"UAF",
					"UAFAnimGraph",
					"UAFAnimNode"
				}
			);
		}
	}
}