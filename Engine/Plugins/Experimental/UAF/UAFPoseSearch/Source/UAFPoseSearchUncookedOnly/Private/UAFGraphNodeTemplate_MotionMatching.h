// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphUncookedOnlyUtils.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/UAFGraphNodeTemplate.h"
#include "MotionMatchingTraitData.h"
#include "HistoryCollectorTraitData.h"
#include "Traits/BlendStackTrait.h"
#include "Traits/BlendSmoother.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "UAFGraphNodeTemplate_MotionMatching.generated.h"

#define LOCTEXT_NAMESPACE "UAFGraphNodeTemplate_MotionMatching"

UCLASS()
class UUAFGraphNodeTemplate_MotionMatching : public UUAFGraphNodeTemplate
{
	GENERATED_BODY()

	UUAFGraphNodeTemplate_MotionMatching()
	{
		Title = LOCTEXT("MotionMatchingTitle", "Motion Matching");
		TooltipText = LOCTEXT("MotionMatchingTooltip", "Performs motion matching on a pose search database with local history collection");
		Category = LOCTEXT("MotionMatchingCategory", "UAF");
		MenuDescription = LOCTEXT("MotionMatchingMenuDesc", "Motion Matching");
		Color = FLinearColor(FColor(29, 96, 125)); // From UE::PoseSearch::GetAssetColor();
		Icon = *FSlateIconFinder::FindIconForClass(UPoseSearchDatabase::StaticClass()).GetIcon();
		Traits =
		{
			TInstancedStruct<FAnimNextBlendStackCoreTraitSharedData>::Make(),
			TInstancedStruct<FAnimNextBlendSmootherCoreTraitSharedData>::Make(),
			TInstancedStruct<FMotionMatchingTraitSharedData>::Make(),
			TInstancedStruct<FAnimNextHistoryCollectorTraitSharedData>::Make()
		};
		DragDropAssetTypes.Add(UPoseSearchDatabase::StaticClass());
		SetCategoryForPinsInLayout(
			{
				GET_PIN_PATH_STRING_CHECKED(FMotionMatchingTraitSharedData, Databases),
				GET_PIN_PATH_STRING_CHECKED(FAnimNextHistoryCollectorTraitSharedData, Trajectory),
			},
			FRigVMPinCategory::GetDefaultCategoryName(),
			NodeLayout,
			true);
	}
};

#undef LOCTEXT_NAMESPACE