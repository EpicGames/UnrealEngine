// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

namespace UE::PoseSearch
{

//////////////////////////////////////////////////////////////////////////
// FPoseSearchEditorModule
class FUAFPoseSearchUncookedOnlyModule : public IModuleInterface
{
};

} // namespace UE::PoseSearch

IMPLEMENT_MODULE(UE::PoseSearch::FUAFPoseSearchUncookedOnlyModule, UAFPoseSearchUncookedOnly)

