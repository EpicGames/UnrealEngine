// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFPoseSearchUncookedOnly : ModuleRules
	{
		public UAFPoseSearchUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"UAF",
					"UAFUncookedOnly",
					"UAFAnimGraph",
					"UAFAnimGraphUncookedOnly",
					"PoseSearch",
					"UAFPoseSearch",
					"RigVMDeveloper",
					"RigVM",
					"Slate",
					"SlateCore", 
					"RewindDebuggerInterface", 
					"PoseSearchEditor",
				}
			);
		}
	}
}