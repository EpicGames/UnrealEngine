{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UAF Pose Search",
	"Description" : "Pose Search integration for UAF.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "Chooser",
			"Enabled": true
		},
		{
			"Name": "EvaluationNotifies",
			"Enabled": true
		},
		{
			"Name": "PoseSearch",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "UAFAnimNode",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "UAFPoseSearch",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFPoseSearchUncookedOnly",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
	]
}
